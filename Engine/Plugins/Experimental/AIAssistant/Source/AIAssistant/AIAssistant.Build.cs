// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AIAssistant : ModuleRules
{
	public AIAssistant(ReadOnlyTargetRules Target) : base(Target)
	{
		// For IWYU audits..
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs; // ..keep
		IWYUSupport = IWYUSupport.Full;  // ..keep, can be None/Minimal/Full
		bUseUnity = true; // ..toggle as needed, use false for IWYU audit


		PublicDefinitions.Add("WITH_AIASSISTANT_EPIC_INTERNAL=1"); 

		
		PublicIncludePaths.AddRange(
			new string[] {
				ModuleDirectory + "/Public",
			}
			);
				
		
		PrivateIncludePaths.AddRange(
			new string[] {
			}
			);
			
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"ToolsetRegistry",
			}
			);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Projects",
				"InputCore",
				"EditorFramework",
				"UnrealEd",
				"ToolMenus",
				"CoreUObject",
				"Engine",
				"DeveloperSettings",
				"Json",
				"JsonUtilities",
				"JsonUtilitiesEditor",
				"EditorSubsystem",
				"HTTP",
                "EditorScriptingUtilities",
                "Kismet",
                "KismetCompiler",
                "BlueprintGraph",
                "GraphEditor",
                "PropertyEditor",
				"UMG",
				"ContentBrowser",
				"AssetRegistry",
				"LevelEditor",
				"StatusBar",
				// For Python execution -
				"PythonScriptPlugin",
				// For Slate input -
				"Slate",
				"SlateCore",
				// For clipboard actions -
				"ApplicationCore",
				// For web browser -
				"WebBrowser"
			}
			);
		
		
		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
			}
			);
	}
}
