// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Containers/Array.h"
#include "Framework/Commands/UICommandList.h"
#include "Modules/ModuleInterface.h"
#include "StatusBarSubsystem.h"
#include "Templates/SharedPointer.h"


#include "AIAssistantCurrentConfig.h"
#include "ToolsetRegistry/DelegateHandle.h"

class FAIAssistantInputProcessor;
class FSpawnTabArgs;
class SAIAssistantWebBrowser;
class SDockTab;
class UAIAssistantSubsystem;


//
// FAIAssistantModule
//


class FAIAssistantModule : public IModuleInterface
{
	friend class UAIAssistantSubsystem;

	
public:

	
	// IModuleInterface interface 
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	

	/**
	 * Get the AI Assistant's web browser widget.
	 * @return The AI Assistant's web browser widget.
	 */
	TSharedPtr<SAIAssistantWebBrowser> GetAIAssistantWebBrowserWidget();


private:

	
	/**
	 * This bring up the plugin tab.
	 */
	void OnOpenPluginTab();

	/**
	 * This bring up the plugin tab but select a drawer if needed based on the cursor location. Close the drawer if already open at the location.
	 */
	void OnTogglePluginTabBasedOnCursorLocation();

	/**
	 * This function will find the best tab manager to open the plugin tab into and open it in a panel drawer if available
	 */
	void OnOpenPluginTab(const class FWidgetPath& InWidgetPath);

	/**
	 * This function will be bound to a Command. It will use AI Assistant to query the Slate UI at the current mouse position.
	 */
	void OnSlateQuery();

	
	/**
	 * DEPRECATED?
	 * Everything left in this method can be done in JavaScript.
	 * 
	 * Pops up a context menu related to the contents in the AI Assistant web browser.
	 * We usually get here via JavaScript, which provides these parameters.
	 * @param SelectedString String for that is selected in web browser.
	 * @param ClientLocation The location where the menu should appear.
	 */
	void ShowContextMenu(const FString& SelectedString, const FVector2f& ClientLocation) const;

	void RegisterMenus();
	
	void RegisterStatusBarPanelDrawerSummon();
	void UnregisterStatusBarPanelDrawerSummon();
	void GenerateStatusBarPanelDrawerSummon(TArray<UStatusBarSubsystem::FTabIdAndButtonLabel>& OutTabIdsAndLabels, const TSharedRef<SDockTab>& InParentTab) const;

	TSharedRef<SDockTab> OnSpawnPluginTab(const FSpawnTabArgs& SpawnTabArgs);
	
	TSharedPtr<UE::AIAssistant::FCurrentConfig> CurrentConfig;
	TSharedPtr<FUICommandList> PluginCommands;
	TSharedPtr<FAIAssistantInputProcessor> InputProcessor;
	TSharedPtr<SAIAssistantWebBrowser> AIAssistantWebBrowserWidget;

	FDelegateHandle StatusBarPanelDrawerSummonHandle;
	UE::ToolsetRegistry::FDelegateHandleRaii OnPostEngineInitHandle;
};
