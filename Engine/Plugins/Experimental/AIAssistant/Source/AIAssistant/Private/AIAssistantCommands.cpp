// Copyright Epic Games, Inc. All Rights Reserved.


#include "AIAssistantCommands.h"

#include "AIAssistantStyle.h"


#define LOCTEXT_NAMESPACE "FAIAssistantModule"


FAIAssistantCommands::FAIAssistantCommands() :
	TCommands<FAIAssistantCommands>(TEXT("AIAssistant"), NSLOCTEXT("Contexts", "AIAssistant", "AIAssistant Plugin"), NAME_None, FAIAssistantStyle::GetStyleSetName())
{
}


/*virtual*/ void FAIAssistantCommands::RegisterCommands()
{
	UI_COMMAND(OpenAIAssistantTab, "AI Assistant", "Open the AI Assistant Tab", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SummonAIAssistantTab, "Summon AI Assistant", "Summon or dismiss the AI Assistant", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Alt, EKeys::F1));
	UI_COMMAND(AISlateQueryCommand, "Query UI with AI Assistant", "Query the UI widget under the mouse with the AI Assistant", EUserInterfaceActionType::Button, FInputChord(EKeys::F1));
}


#undef LOCTEXT_NAMESPACE
