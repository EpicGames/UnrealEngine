// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"

class FAIAssistantCommands : public TCommands<FAIAssistantCommands>
{
public:
	FAIAssistantCommands();	

	// TCommands<> interface
	virtual void RegisterCommands() override;

	// IMPORTANT: The variable name of a command installed in a menu ends up being used as the menu
	// item name, for example this is installed by AIAssistant.cpp as
	// "LevelEditor.MainMenu.Window.OpenAIAssistantTab".

	TSharedPtr<FUICommandInfo> OpenAIAssistantTab;
	TSharedPtr<FUICommandInfo> SummonAIAssistantTab;
	TSharedPtr<FUICommandInfo> AISlateQueryCommand;
};