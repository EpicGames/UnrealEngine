// Copyright Epic Games, Inc. All Rights Reserved.

#include "AIAssistantConfig.h"

#include "Containers/Set.h"
#include "Misc/FileHelper.h"
#include "Misc/Optional.h"
#include "Misc/Paths.h"
#include "Templates/UnrealTemplate.h"

#include "AIAssistantLog.h"

namespace UE::AIAssistant
{
	const FString FConfig::DefaultFilename("AIAssistant.json");

	const FString FConfig::DefaultMainUrl(
		"https://dev.epicgames.com/community/assistant/embedded");

	TArray<FString> FConfig::GetDefaultSearchDirectories()
	{
		TArray<FString> BaseSearchPaths =
		{
			FPaths::EngineConfigDir(),
			FPaths::EngineUserDir(),
			FPaths::EngineVersionAgnosticUserDir(),
		};
		TArray<FString> AllPathsToSearch;
		TArray<TOptional<FPaths::EPathConversion>> PathConversions =
		{
			TOptional(FPaths::EPathConversion::Engine_NotForLicensees),
			TOptional(FPaths::EPathConversion::Engine_NoRedist),
			TOptional(FPaths::EPathConversion::Engine_LimitedAccess),
			TOptional<FPaths::EPathConversion>(),
		};
		for (const TOptional<FPaths::EPathConversion>& PathConversion : PathConversions)
		{
			for (const FString& BaseSearchPath : BaseSearchPaths)
			{
				AllPathsToSearch.Emplace(
					PathConversion.IsSet()
					? FPaths::ConvertPath(BaseSearchPath, PathConversion.GetValue())
					: BaseSearchPath);
			}

		}
		return AllPathsToSearch;
	}

	FString FConfig::FindConfigFile(const TArray<FString>& SearchDirectories)
	{
		for (auto SearchDirectory : SearchDirectories)
		{
			auto FullFilename = FPaths::Combine(SearchDirectory, DefaultFilename);
			UE_LOGF(
				LogAIAssistant, Verbose, "Searching for AI assistant config in %ls",
				*FullFilename);
			if (FPaths::FileExists(FullFilename))
			{
				return FullFilename;
			}
		}
		return FString();
	}

	FConfig FConfig::Load(const FString& Filename)
	{
		FString Json(TEXT("{}"));
		if (!Filename.IsEmpty() && !FFileHelper::LoadFileToString(Json, *Filename))
		{
			UE_LOGF(
				LogAIAssistant, Error, "Failed to load AI assistant config from \"%ls\".",
				*Filename);
		}
		FConfig Config;
		if (!Config.FromJson(Json))
		{
			UE_LOGF(
				LogAIAssistant, Error, "Failed to load AI assistant config \"%ls\" from \"%ls\".",
				*Json, *Filename);
			// Use the default config.
			(void)Config.FromJson(FString());
		}
		return Config;
	}
}