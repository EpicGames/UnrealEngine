// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Serialization/JsonSerializable.h"
#include "Serialization/JsonSerializerMacros.h"

namespace UE::AIAssistant
{
	// Internal configuration for developers of the AI assistant.
	struct FConfig : public FJsonSerializable
	{
		// Initial URL loaded when starting the assistant.
		// If this isn't specified or is empty, FConfig::DefaultMainUrl is used.
		FString MainUrl;
		// JSON dict of arbitrary string-any pairs for development options.
		FString DevOptionsRawJson;

		BEGIN_JSON_SERIALIZER
			JSON_SERIALIZE_WITHDEFAULT("main_url", MainUrl, DefaultMainUrl)
			JSON_SERIALIZE_RAW_JSON_STRING("dev_options", DevOptionsRawJson)
		END_JSON_SERIALIZER

		// Get the default configuration search directories with the most specific overrides first.
		//
		// For example, when searching the engine configuration directory:
		// * Engine/Restricted/NotForLicensees/Config/
		// * Engine/Restricted/NoRedist/Config/
		// * Engine/Restricted/LimitedAccess/Config/
		// * Engine/Config/
		//
		// The base directories that are searched with overrides are:
		// * Engine configuration directory: Engine/Config/
		// * Engine directory: Engine/
		// * Engine user directory (if installed): On Windows %LOCALAPPDATA%/UnrealEngine/Common/
		static TArray<FString> GetDefaultSearchDirectories();

		// Search for the assistant configuration file in the specified set of directories.
		static FString FindConfigFile(const TArray<FString>& SearchDirectories);

		// Attempt to the load the configuration from the specified filename. If the filename is empty
		// return the default configuration. If configuration loading fails, an error is logged and the
		// default configuration is returned.
		static FConfig Load(const FString& Filename);

		// Load the configuration from a file by searching default search directories.
		static FConfig Load()
		{
			return Load(FindConfigFile(GetDefaultSearchDirectories()));
		}

		// Default initial URL when starting the assistant.
		static const FString DefaultMainUrl;

		// Configuration filename (basename, not the full path).
		static const FString DefaultFilename;
	};

}