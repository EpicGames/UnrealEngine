// Copyright Epic Games, Inc. All Rights Reserved.

#include "AIAssistantConversationReadyExecutor.h"

#include "Async/Mutex.h"
#include "Async/UniqueLock.h"
#include "Templates/Function.h"
#include "Templates/UnrealTemplate.h"

namespace UE::AIAssistant
{
	FConversationReadyExecutor::FConversationReadyExecutor(
		FGetExecuteWhenReadyStateFunc&& GetExecuteWhenReadyStateFunc) :
			GetExecuteWhenReadyStateDelegate(MoveTemp(GetExecuteWhenReadyStateFunc))
	{
		Reset();
	}

	void FConversationReadyExecutor::Reset()
	{
		UE::TUniqueLock Lock(StateMutex);
		bConfiguredAgentEnvironment = false;
		bCreatingConversation = false;
	}

	void FConversationReadyExecutor::NotifyAgentEnvironmentConfigured()
	{
		{
			UE::TUniqueLock Lock(StateMutex);
			bConfiguredAgentEnvironment = true;
		}
		UpdateExecuteWhenReady();
	}

	void FConversationReadyExecutor::NotifyAgentEnvironmentReconfiguring()
	{
		{
			UE::TUniqueLock Lock(StateMutex);
			bConfiguredAgentEnvironment = false;
		}
	}

	bool FConversationReadyExecutor::SetCreatingConversation(bool bNewCreatingConversation)
	{
		bool bPreviousCreatingConversation;
		bool bCreatingNewConversation;
		bool bCreatedNewConversation;
		{
			UE::TUniqueLock Lock(StateMutex);
			bPreviousCreatingConversation = bCreatingConversation;
			bCreatingConversation = bNewCreatingConversation;
			bCreatingNewConversation = !bPreviousCreatingConversation && bCreatingConversation;
			bCreatedNewConversation = bPreviousCreatingConversation && !bCreatingConversation;
		}
		if (bCreatingNewConversation)
		{
			ResetExecuteWhenReady();
		}
		if (bCreatedNewConversation)
		{
			UpdateExecuteWhenReady();
		}
		return bPreviousCreatingConversation;
	}

	FExecuteWhenReady::EExecuteWhenReadyState FConversationReadyExecutor::GetExecuteWhenReadyState()
	{
		if (GetExecuteWhenReadyStateDelegate)
		{
			auto ExecuteWhenReadyState = GetExecuteWhenReadyStateDelegate();
			if (ExecuteWhenReadyState != EExecuteWhenReadyState::Execute)
			{
				return ExecuteWhenReadyState;
			}
		}
		UE::TUniqueLock Lock(StateMutex);
		return !bConfiguredAgentEnvironment || bCreatingConversation
			? EExecuteWhenReadyState::Wait
			: EExecuteWhenReadyState::Execute;
	}

	void FConversationReadyExecutor::ClearExecutionQueueIfCreatingConversation()
	{
		UE::TUniqueLock Lock(StateMutex);
		if (bCreatingConversation)
		{
			ResetExecuteWhenReady();
		}
	}
}