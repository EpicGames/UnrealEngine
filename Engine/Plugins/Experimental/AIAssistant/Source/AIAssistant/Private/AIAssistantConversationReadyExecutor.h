// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Async/Mutex.h"
#include "Templates/Function.h"
#include "Templates/UnrealTemplate.h"

#include "AIAssistantExecuteWhenReady.h"

namespace UE::AIAssistant
{
	// Handles enqueuing of operations against a conversation until it's ready to accept messages.
	class FConversationReadyExecutor : public FExecuteWhenReady
	{
	public:
		using FGetExecuteWhenReadyStateFunc = TFunction<FExecuteWhenReady::EExecuteWhenReadyState()>;

	public:
		// Construct with a delegate that retrieves additional execution when ready state.
		FConversationReadyExecutor(
			FGetExecuteWhenReadyStateFunc&& GetExecuteWhenReadyStateDelegate =
				FGetExecuteWhenReadyStateFunc());

		virtual ~FConversationReadyExecutor() = default;

		// Reset to the initial state.
		void Reset();

		// Notify the executor that the agent environment has been configured.
		void NotifyAgentEnvironmentConfigured();

		// Notify the executor that the agent environment is being reconfigured.
		void NotifyAgentEnvironmentReconfiguring();

		// Whether the agent environment has been configured.
		bool IsAgentEnvironmentConfigured() const { return bConfiguredAgentEnvironment; }

		// Set the creating conversation flag returning the previous value.
		bool SetCreatingConversation(bool bNewCreatingConversation);

		template<class CallableType>
		void ExecuteWhenReady(CallableType&& Callable)
		{
			ClearExecutionQueueIfCreatingConversation();
			FExecuteWhenReady::ExecuteWhenReady(Forward<CallableType>(Callable));
		}

	protected:
		FExecuteWhenReady::EExecuteWhenReadyState GetExecuteWhenReadyState() override;

	private:
		void ClearExecutionQueueIfCreatingConversation();

	private:
		mutable UE::FMutex StateMutex;
		bool bConfiguredAgentEnvironment;
		bool bCreatingConversation;
		FGetExecuteWhenReadyStateFunc GetExecuteWhenReadyStateDelegate;

	};
}