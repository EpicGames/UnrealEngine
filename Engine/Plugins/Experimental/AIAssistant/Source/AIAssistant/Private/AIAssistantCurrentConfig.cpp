// Copyright Epic Games, Inc. All Rights Reserved.

#include "AIAssistantCurrentConfig.h"

#include "CoreGlobals.h"
#include "Misc/ConfigCacheIni.h"

#include "AIAssistantConfig.h"

namespace UE::AIAssistant
{
	FCurrentConfig::FCurrentConfig(const FConstructorArgs& Args) :
		bIsExiting(false),
		OnPreExitDelegate(Args.OnPreExit ? *Args.OnPreExit : FCoreDelegates::OnPreExit),
		OnPreExitDelegateHandle(
			UE::ToolsetRegistry::FDelegateHandleRaii::Create(
				OnPreExitDelegate,
				OnPreExitDelegate.AddRaw(this, &FCurrentConfig::OnPreExitHandler)))
	{
		GConfig->GetBool(
			TEXT("AIAssistant"), TEXT("bIsEnabled"), bIsEnabled, Args.EditorIniFilename);
	}

	TSharedPtr<const FConfig> FCurrentConfig::Load()
	{
		auto LoadedConfig = MakeShared<const FConfig>(FConfig::Load());
		Config = LoadedConfig;
		return LoadedConfig;
	}

	TSharedPtr<const FConfig> FCurrentConfig::GetOrLoad()
	{
		auto LoadedConfig = Config;
		if (LoadedConfig) return LoadedConfig;
		return Load();
	}

	void FCurrentConfig::OnPreExitHandler() { bIsExiting.store(true); }
}