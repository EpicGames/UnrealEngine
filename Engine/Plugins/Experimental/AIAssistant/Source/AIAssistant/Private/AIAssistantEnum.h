// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include <cstddef>
#include <tuple>

#include "Containers/StaticArray.h"
#include "Containers/UnrealString.h"
#include "Misc/Optional.h"

// IMPORTANT: If we want to continue using this it should probably live somewhere else.
namespace UE::AIAssistant
{
	// NOTE: This should *only* be used in the rare circumstances that UENUM() is not appropriate.
	// For example, if you want to have control over serialization of strings associated with
	// enum values.

	// Used to build an array of enum values.
	//
	// Given a macro that defines enum values like this:
	// #define MY_ENUM(X)      \
	//    X(Hello, "Hiya"),    \
	//    X(Goodbye, "See-ya")
	//
	// The following generates an enum with Hello and Goodbye members:
	// enum class ESalutation { MY_ENUM(UE_ENUM_VALUE) };
	#define UE_ENUM_VALUE(Value, UnusedDescription) Value

	// Used to build an array of EnumValueDescription.
	//
	//  Given a macro that defines enum values like this:
	// #define MY_ENUM(X)      \
	//    X(Hello, "Hiya"),    \
	//    X(Goodbye, "See-ya")
	//
	// The following generates an array with enums mapped to strings:
	//    EnumValueDescription Description[] = { MY_ENUM(UE_ENUM_VALUE_DESCRIPTION) };
	#define UE_ENUM_VALUE_DESCRIPTION(Value, Description) \
	  UE::AIAssistant::EnumValueDescription{ Value, FString(TEXT(Description)) }

	// Used to count the number of entries in an enum.
	//
	// Given a macro that defines enum values like this:
	// #define MY_ENUM(X) \
	//	 X(Hello, "Hiya"), \
	//	 X(Goodbye, "See-ya")
	//
	// The following generates a count of enum values stored in MyEnumCount.
	// constexpr uint32 MyEnumCount =
	//     UE_ENUM_COUNT(MY_ENUM(UE_ENUM_COUNTER));
	//
	// The macro that defines an enum must *not* include a trailing comma.
	#define UE_ENUM_COUNT(...) \
		std::tuple_size<decltype(std::make_tuple(__VA_ARGS__))>::value

	// See UE_ENUM_COUNT().
	#define UE_ENUM_COUNTER(UnusedValue, UnusedDescription) 1

	// Expands to an array type for the specified enum.
	//
	// Given a macro that defines enum values like this for enum MyEnum:
	// #define MY_ENUM(X) \
	//     X(Hello, "Hiya"), \
	//     X(Goodbye, "See-ya")
	//
	// The following declares an EnumValueDescription array.
	// extern const UE_ENUM_VALUE_DESCRIPTION_TYPE(EMyEnum, MY_ENUM MyEnumDescription);
	#define UE_ENUM_VALUE_DESCRIPTION_TYPE(EnumType, EnumTypeMacro) \
		TStaticArray< \
			UE::AIAssistant::EnumValueDescription<EnumType>, \
			UE_ENUM_COUNT(EnumTypeMacro(UE_ENUM_COUNTER))>

	// Declares metadata for an enum in the current namespace. (use in a header)
	//
	// * EnumTypeCount constant containing the number of items in the enum.
	// * EnumTypeDescriptions constant containing the map of enum values to strings.
	// * LexFromString(EnumType& OutputValue, const TCHAR* String)
	//   to convert from a string to a value.
	// * FString LexToString(EnumType Value) to convert from an enum value to string.
	#define UE_ENUM_METADATA_DECLARE(EnumType, EnumDescriptionMacro)                        \
		/* Number of members of EnumType */                                                 \
		extern const uint32 EnumType##Count;                                           \
		                                                                                    \
		/* Description of each member of EnumType */                                        \
		extern const UE_ENUM_VALUE_DESCRIPTION_TYPE(EnumType, EnumDescriptionMacro)         \
			EnumType##Descriptions;                                                         \
		                                                                                    \
		/* Convert from a string to an EnumType's member value. */                          \
		void LexFromString(EnumType& OutputValue, const TCHAR* String);                     \
		                                                                                    \
		/* Convert from an EnumType's member value to a string. */                          \
		FString LexToString(EnumType Value);

	// Defines metadata for an enum in the current namespace. (use in a source file)
	#define UE_ENUM_METADATA_DEFINE(EnumType, EnumDescriptionMacro)                         \
		const uint32 EnumType##Count =                                                 \
			UE_ENUM_COUNT(EnumDescriptionMacro(UE_ENUM_COUNTER));                           \
		                                                                                    \
		const UE_ENUM_VALUE_DESCRIPTION_TYPE(EnumType, EnumDescriptionMacro)                \
			EnumType##Descriptions =                                                        \
		{                                                                                   \
			EnumDescriptionMacro(UE_ENUM_VALUE_DESCRIPTION)                                 \
		};                                                                                  \
		                                                                                    \
		void LexFromString(EnumType& OutputValue, const TCHAR* String)                      \
		{                                                                                   \
			auto MaybeValue = UE::AIAssistant::GetEnumValueFromDescription(                 \
				EnumType##Descriptions, FString(String));                                   \
			if (MaybeValue.IsSet()) OutputValue = *MaybeValue;                              \
		}                                                                                   \
		                                                                                    \
		FString LexToString(EnumType Value)                                                 \
		{                                                                                   \
			auto MaybeDescription = UE::AIAssistant::GetEnumValueDescription(                \
				EnumType##Descriptions, Value);                                             \
			return MaybeDescription.IsSet() ? **MaybeDescription : FString();               \
		}

	// Describes an enum value.
	template <typename T>
	struct EnumValueDescription
	{
		T Value;
		const FString Description;
	};

	// Get the description of an enum value.
	template <typename T, auto Size>
	TOptional<const FString*> GetEnumValueDescription(
		const TStaticArray<EnumValueDescription<T>, Size>& EnumValueDescriptions, T EnumValue)
	{
		TOptional<const FString*> ReturnValue;
		for (auto& EnumValueDescription : EnumValueDescriptions)
		{
			if (EnumValueDescription.Value == EnumValue)
			{
				ReturnValue.Emplace(&EnumValueDescription.Description);
				break;
			}
		}
		return ReturnValue;
	}

	// Convert an enum description to an enum value returning no value if the description is not found.
	template <typename T, auto Size>
	TOptional<T> GetEnumValueFromDescription(
		const TStaticArray<EnumValueDescription<T>, Size>& EnumValueDescriptions,
		const FString& Description, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive)
	{
		TOptional<T> ReturnValue;
		for (auto& EnumValueDescription : EnumValueDescriptions)
		{
			if (EnumValueDescription.Description.Compare(Description, SearchCase) == 0)
			{
				ReturnValue.Emplace(EnumValueDescription.Value);
				break;
			}
		}
		return ReturnValue;
	}

}  // namespace UE::AIAssistant
