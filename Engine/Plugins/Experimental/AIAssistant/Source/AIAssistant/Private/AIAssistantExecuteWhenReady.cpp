// Copyright Epic Games, Inc. All Rights Reserved.
	

#include "AIAssistantExecuteWhenReady.h"

#include "Misc/ScopeRWLock.h"
#include <functional>


UE::AIAssistant::FExecuteWhenReady::FExecuteWhenReady(FExecuteWhenReady&& Other) noexcept
{
	MoveFrom(Other);
}


UE::AIAssistant::FExecuteWhenReady& UE::AIAssistant::FExecuteWhenReady::operator=(FExecuteWhenReady&& RHS) noexcept
{
	MoveFrom(RHS);
	
	return *this;
}


void UE::AIAssistant::FExecuteWhenReady::Enqueue(FDeferredExecutionFunction&& DeferredExecutionFunction)
{
	if (GetExecuteWhenReadyState() == EExecuteWhenReadyState::Execute)
	{
		DeferredExecutionFunction();
	}
	else 
	{
		FWriteScopeLock WriteScopeLock(DeferredExecutionFunctionRWLock);
		
		DeferredExecutionFunctions.Add(MoveTemp(DeferredExecutionFunction));
	}
}


void UE::AIAssistant::FExecuteWhenReady::UpdateExecuteWhenReady()
{
	if (GetExecuteWhenReadyState() == EExecuteWhenReadyState::Execute)
	{
		FWriteScopeLock WriteScopeLock(DeferredExecutionFunctionRWLock);

		for (TArray<FDeferredExecutionFunction> Functions = MoveTemp(DeferredExecutionFunctions);
			FDeferredExecutionFunction& Function : Functions)
		{
			Function();
		}
	}
	else if (GetExecuteWhenReadyState() == EExecuteWhenReadyState::Reject)
	{
		ResetExecuteWhenReady();
	}
}


void UE::AIAssistant::FExecuteWhenReady::ResetExecuteWhenReady()
{
	FWriteScopeLock WriteScopeLock(DeferredExecutionFunctionRWLock);

	DeferredExecutionFunctions.Empty();
}


int32 UE::AIAssistant::FExecuteWhenReady::GetNumDeferredExecutionFunctions() const
{
	FReadScopeLock ReadScopedLock(DeferredExecutionFunctionRWLock);

	return DeferredExecutionFunctions.Num();
}


bool UE::AIAssistant::FExecuteWhenReady::IsExecuteWhenReadyPending() const
{
	return GetNumDeferredExecutionFunctions() > 0;
}


void UE::AIAssistant::FExecuteWhenReady::MoveFrom(FExecuteWhenReady& Other) noexcept
{
	if (this == &Other)
	{
		return;
	}

	
	// Looks strange, but we're being extra careful to avoid a particular kind of deadlock, which would be SUPER HARD to debug. Use pointer values to
	// find a consistent locking order - in the very unlikely case that one thread tries to move two of these classes, while another thread tries to
	// move the same two classes at the same time but in the opposite order. This way, they can't both acquire a first lock that is the second lock
	// of the other one, with both would ending up getting blocked trying to acquire their second lock. To enforce the lock order, make the lower
	// pointer value always lock first, and the higher pointer value always lock second. (It's technically safest to use std::less for the
	// comparison.)
	
	const bool bLockThisBeforeOther = std::less<const FExecuteWhenReady*>()(this, &Other); // ..see comment
	FWriteScopeLock WriteScopeLock1((bLockThisBeforeOther ? this : &Other)->DeferredExecutionFunctionRWLock);
	FWriteScopeLock WriteScopeLock2((bLockThisBeforeOther ? &Other : this)->DeferredExecutionFunctionRWLock);


	// We want to move everything, but are not allowed to move the lock.
	
	DeferredExecutionFunctions = MoveTemp(Other.DeferredExecutionFunctions);
}
