// Copyright Epic Games, Inc. All Rights Reserved.
	

#pragma once


#include "Templates/Function.h"
#include "Containers/Array.h"
#include "HAL/CriticalSection.h"


//
// UE::AIAssistant::FExecuteWhenReady
//
// This class provides general inheritable functionality to defer the execution of callables until a class declares that it's ready to do so. This
// avoids someone having to check state in something like a Tick() function, before executing certain functionality that depends on a particular
// condition. 
//
// A class that inherits from this class -
//		- can have callables added to it via ExecuteWhenReady() , which will be executed, in order, once the class says its ready
//		- should itself override GetExecuteWhenReadyState() to notify its ready state
//		- should itself call UpdateExecuteWhenReady() when necessary to execute the callables provided, according to its ready state
//
// Example -
// A class that uses a web browser may need to wait until a web page is loaded before executing certain functions, such as sending data to that web
// page. If that class inherits from this class, those functions can be added as callables to it. That class can monitor whether the web page is
// loaded. Whenever web pages are done loading, it would set it's ready state and then can call the update function.  
//


namespace UE::AIAssistant
{
	class FExecuteWhenReady
	{
	public:

		
		FExecuteWhenReady() = default;
		virtual ~FExecuteWhenReady() = default;

		// Disallow copies. (Unique functions in here are move-only.)
		FExecuteWhenReady(const FExecuteWhenReady&) = delete;
		FExecuteWhenReady& operator=(const FExecuteWhenReady&) = delete;

		// Allow moves.
		FExecuteWhenReady(FExecuteWhenReady&&) noexcept;
		FExecuteWhenReady& operator=(FExecuteWhenReady&&) noexcept;


		/**
		 * Internal type of deferred execution function.
		 */
		using FDeferredExecutionFunction = TUniqueFunction<void()>;
		

		/**
		 * To provide any callable that should be executed if/when this class is ready to do so.
		 * @param Callable Any callable, i.e. a lambda: []()->void{..do something..}
		 */
		template<class CallableType>
		void ExecuteWhenReady(CallableType&&/*forward ref!*/ Callable)
		{
			Enqueue(FDeferredExecutionFunction(Forward<CallableType>(Callable)));
		}


		enum class EExecuteWhenReadyState : uint8
		{
			Wait = 0,
			Execute,
			Reject
		};

		/**
		 * Determine ready state. Overridden by derived class.
		 * @return Ready state.
		 */
		virtual EExecuteWhenReadyState GetExecuteWhenReadyState() = 0;


		/**
		 * Calls and clears deferred execution functions if ready state is set to execute.
		 * Clears deferred execution functions if ready state is set to reject.
		 * Otherwise ignores deferred execution functions for now.
		 */
		void UpdateExecuteWhenReady();

		/**
		 * Clear deferred execution functions.
		 */
		void ResetExecuteWhenReady();


		/**
		 * How many deferred execution functions are stored? Will be 0 after execution or rejection.
		 * @return Current number of deferred execution functions stored. 
		 */
		int32 GetNumDeferredExecutionFunctions() const;

		/**
		 * Tells us if there are any deferred execution functions waiting to execute.
		 * @return If there are any deferred execution functions waiting to execute.
		 */
		bool IsExecuteWhenReadyPending() const;
		

	private:
		

		// Used during access/modification of deferred execution functions.
		mutable FRWLock DeferredExecutionFunctionRWLock;

		// Either executes incoming deferred execution function if ready now, or saves it to execute when ready later.
		void Enqueue(FDeferredExecutionFunction&& DeferredExecutionFunction);

		// Deferred functions to execute in order, when we become ready to execute them.
		TArray<FDeferredExecutionFunction> DeferredExecutionFunctions;

		// Used for custom moves.
		void MoveFrom(FExecuteWhenReady& Other) noexcept;
	};
}
