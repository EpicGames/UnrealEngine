// Copyright Epic Games, Inc. All Rights Reserved.

#include "AIAssistantFileLockManager.h"

#include "AIAssistantLog.h"
#include "Editor.h"
#include "Containers/Ticker.h"
#include "AssetTypeActivationOpenedMethod.h"
#include "HAL/IConsoleManager.h"
#include "Subsystems/AssetEditorSubsystem.h"

namespace
{
	// Encapsulates all mutable state for the file lock manager.
	struct FFileLockManagerState
	{
		// Mutex protecting LockedFiles.
		UE::FMutex Mutex;

		// Set of locked package paths.
		TSet<FString> LockedFiles;

		// Handle for the OnAssetOpenedInEditor delegate.
		FDelegateHandle AssetOpenedHandle;

		void Reset()
		{
			UE::TUniqueLock Lock(Mutex);
			LockedFiles.Empty();
			AssetOpenedHandle.Reset();
		}
	};

	static FFileLockManagerState FileLockManagerState;

	static FString NormalizePath(const FString& PackagePath)
	{
		FString NormalizedPath = PackagePath.TrimStartAndEnd();
		if (NormalizedPath.IsEmpty())
		{
			return NormalizedPath;
		}
		if (!NormalizedPath.StartsWith(TEXT("/")))
		{
			NormalizedPath = TEXT("/") + NormalizedPath;
		}
		return NormalizedPath;
	}

	static void OnAssetOpenedInEditor(UObject* Asset, IAssetEditorInstance* EditorInstance)
	{
		using UE::AIAssistant::FFileLockManager;

		if (!Asset || !EditorInstance)
		{
			return;
		}

		UPackage* Package = Asset->GetPackage();
		if (!Package || !FFileLockManager::IsFileLocked(Package->GetName()))
		{
			return;
		}

		// If already in View mode, nothing to do.
		if (EditorInstance->GetOpenMethod() == EAssetOpenMethod::View)
		{
			return;
		}

		// Asset is locked but opened for editing: defer close and reopen in View mode
		// to avoid modifying editor state during the notification callback.
		FTSTicker::GetCoreTicker().AddTicker(
			FTickerDelegate::CreateLambda(
				[WeakAsset = TWeakObjectPtr<UObject>(Asset)](float) -> bool
				{
					UObject* AssetPtr = WeakAsset.Get();
					if (!GEditor || !AssetPtr)
					{
						return false;
					}
					UAssetEditorSubsystem* Subsystem =
						GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
					if (!Subsystem)
					{
						return false;
					}
					// Close and reopen the asset in View mode.
					// If the user has edits to the file they will not be lost, they will remain
					// in-memory and be reflected when the asset editor is reopened.
					Subsystem->CloseAllEditorsForAsset(AssetPtr);
					bool bShowProgressWindow = true;
					Subsystem->OpenEditorForAsset(
						AssetPtr, EToolkitMode::Standalone, TSharedPtr<IToolkitHost>(),
						bShowProgressWindow, EAssetTypeActivationOpenedMethod::View);
					
					return false;
				}),
			0.0f);
	}
}  // namespace

namespace UE::AIAssistant
{

bool FFileLockManager::AddLockedFile(const FString& PackagePath)
{
	FString NormalizedPath = NormalizePath(PackagePath);
	if (NormalizedPath.IsEmpty())
	{
		return false;
	}

	UE::TUniqueLock Lock(FileLockManagerState.Mutex);
	bool bAlreadyInSet = false;
	FileLockManagerState.LockedFiles.Add(NormalizedPath, &bAlreadyInSet);
	return !bAlreadyInSet;
}

bool FFileLockManager::RemoveLockedFile(const FString& PackagePath)
{
	FString NormalizedPath = NormalizePath(PackagePath);
	if (NormalizedPath.IsEmpty())
	{
		return false;
	}

	UE::TUniqueLock Lock(FileLockManagerState.Mutex);
	return FileLockManagerState.LockedFiles.Remove(NormalizedPath) > 0;
}

bool FFileLockManager::IsFileLocked(const FString& PackagePath)
{
	FString NormalizedPath = NormalizePath(PackagePath);

	UE::TUniqueLock Lock(FileLockManagerState.Mutex);

	// Check exact match and prefix match (for sub-assets)
	for (const FString& LockedPath : FileLockManagerState.LockedFiles)
	{
		if (NormalizedPath == LockedPath || NormalizedPath.StartsWith(LockedPath + TEXT(".")))
		{
			return true;
		}
	}

	return false;
}

TSet<FString> FFileLockManager::GetLockedFiles()
{
	UE::TUniqueLock Lock(FileLockManagerState.Mutex);
	return FileLockManagerState.LockedFiles;
}

void FFileLockManager::ClearLockedFiles()
{
	UE::TUniqueLock Lock(FileLockManagerState.Mutex);
	FileLockManagerState.LockedFiles.Empty();
}

void FFileLockManager::RegisterWithEditor()
{
	if (!GEditor)
	{
		UE_LOGF(LogAIAssistant, Warning, "RegisterWithEditor called without a valid GEditor");
		return;
	}

	if (FileLockManagerState.AssetOpenedHandle.IsValid())
	{
		return;
	}
	UAssetEditorSubsystem* AssetEditorSubsystem =
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
	if (!AssetEditorSubsystem)
	{
		UE_LOGF(LogAIAssistant, Warning, "Could not access AssetEditorSubsystem");
		return;
	}

	// When an asset opens in an editor, check if it's locked and force View mode
	FileLockManagerState.AssetOpenedHandle =
		AssetEditorSubsystem->OnAssetOpenedInEditor().AddStatic(&OnAssetOpenedInEditor);
}

void FFileLockManager::UnregisterFromEditor()
{
	if (!GEditor)
	{
		return;
	}

	UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
	if (AssetEditorSubsystem)
	{
		AssetEditorSubsystem->OnAssetOpenedInEditor().Remove(FileLockManagerState.AssetOpenedHandle);
	}

	FileLockManagerState.Reset();
}

}  // namespace UE::AIAssistant

// Console commands for testing and debugging
namespace
{

void LockFileCommand(const TArray<FString>& Args)
{
	using UE::AIAssistant::FFileLockManager;

	if (Args.Num() == 0)
	{
		UE_LOGF(
			LogAIAssistant, Warning, 
			"AIAssistant.LockFile: Please provide a package path. "
				 "Usage: AIAssistant.LockFile <PackagePath>");
		return;
	}

	FString PackagePath = Args[0];

	if (FFileLockManager::AddLockedFile(PackagePath))
	{
		UE_LOGF(
			LogAIAssistant, Log,
			"AIAssistant.LockFile: Successfully locked file: %ls", *PackagePath);
	}
	else
	{
		UE_LOGF(
			LogAIAssistant, Warning,
			"AIAssistant.LockFile: File is already locked: %ls", *PackagePath);
	}
}

void UnlockFileCommand(const TArray<FString>& Args)
{
	using UE::AIAssistant::FFileLockManager;

	if (Args.Num() == 0)
	{
		UE_LOGF(
			LogAIAssistant, Warning,
			"AIAssistant.UnlockFile: Please provide a package path. "
				 "Usage: AIAssistant.UnlockFile <PackagePath>");
		return;
	}

	FString PackagePath = Args[0];

	if (FFileLockManager::RemoveLockedFile(PackagePath))
	{
		UE_LOGF(
			LogAIAssistant, Log,
			"AIAssistant.UnlockFile: Successfully unlocked file: %ls", *PackagePath);
	}
	else
	{
		UE_LOGF(
			LogAIAssistant, Warning,
			"AIAssistant.UnlockFile: File was not locked: %ls", *PackagePath);
	}
}

void ListLockedFilesCommand()
{
	const TSet<FString> LockedFiles = UE::AIAssistant::FFileLockManager::GetLockedFiles();

	if (LockedFiles.Num() == 0)
	{
		UE_LOGF(
			LogAIAssistant, Log,
			"AIAssistant.ListLockedFiles: No files are currently locked.");
	}
	else
	{
		UE_LOGF(
			LogAIAssistant, Log,
			"AIAssistant.ListLockedFiles: %d file(s) are locked:", LockedFiles.Num());
		for (const FString& FilePath : LockedFiles)
		{
			UE_LOGF(LogAIAssistant, Log, "  - %ls", *FilePath);
		}
	}
}

void ClearLockedFilesCommand()
{
	UE::AIAssistant::FFileLockManager::ClearLockedFiles();
	UE_LOGF(
		LogAIAssistant, Log, "AIAssistant.ClearLockedFiles: All locked files have been cleared.");
}

FAutoConsoleCommand CCmdLockFile(
	TEXT("AIAssistant.LockFile"),
	TEXT("Adds a file to the locked files list. Usage: AIAssistant.LockFile <PackagePath>"),
	FConsoleCommandWithArgsDelegate::CreateStatic(&LockFileCommand)
);

FAutoConsoleCommand CCmdUnlockFile(
	TEXT("AIAssistant.UnlockFile"),
	TEXT("Removes a file from the locked files list. Usage: AIAssistant.UnlockFile <PackagePath>"),
	FConsoleCommandWithArgsDelegate::CreateStatic(&UnlockFileCommand)
);

FAutoConsoleCommand CCmdListLockedFiles(
	TEXT("AIAssistant.ListLockedFiles"),
	TEXT("Lists all currently locked files."),
	FConsoleCommandDelegate::CreateStatic(&ListLockedFilesCommand)
);

FAutoConsoleCommand CCmdClearLockedFiles(
	TEXT("AIAssistant.ClearLockedFiles"),
	TEXT("Clears all locked files from the list."),
	FConsoleCommandDelegate::CreateStatic(&ClearLockedFilesCommand)
);

}  // namespace
