// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace UE::AIAssistant
{

// Manages a list of files that should be locked from editing in the Unreal Editor.
// Files in this list will be opened in read-only mode.
class FFileLockManager
{
public:
	// Add a file to the locked files list. Returns true if added, false if already present.
	static bool AddLockedFile(const FString& PackagePath);

	// Remove a file from the locked files list. Returns true if removed, false if not found.
	static bool RemoveLockedFile(const FString& PackagePath);

	// Check if a file is locked by package path (e.g., "/Game/MyAsset").
	static bool IsFileLocked(const FString& PackagePath);

	// Get all locked files.
	static TSet<FString> GetLockedFiles();

	// Clear all locked files.
	static void ClearLockedFiles();

	// Register file locking hooks with the editor (asset open callback and read-only filter).
	static void RegisterWithEditor();

	// Unregister file locking hooks from the editor.
	static void UnregisterFromEditor();
};

}  // namespace UE::AIAssistant
