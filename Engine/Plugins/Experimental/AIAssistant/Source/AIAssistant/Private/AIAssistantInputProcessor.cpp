// Copyright Epic Games, Inc. All Rights Reserved.


#include "AIAssistantInputProcessor.h"

#include "IDetailsView.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Toolkits/BaseToolkit.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Docking/SDockTab.h"


#define LOCTEXT_NAMESPACE "FAIAssistantInputProcessor"


//
// FAIAssistantInputProcessor
//


FAIAssistantInputProcessor::FAIAssistantInputProcessor(const TSharedPtr<FUICommandList>& InCommands) :
	Commands(InCommands)
{
	// See all NOTE_AI_ASSISTANT_INPUT_PROCESSOR.
}


/*virtual*/ void FAIAssistantInputProcessor::Tick(const float DeltaTime, FSlateApplication& SlateApp, TSharedRef<ICursor> Cursor)
{
	// Required to make this class non-abstract. No Super call available.
}


/*virtual*/ bool FAIAssistantInputProcessor::HandleKeyDownEvent(FSlateApplication& SlateApp, const FKeyEvent& KeyEvent) /*override*/
{
	// See all NOTE_AI_ASSISTANT_INPUT_PROCESSOR.
	
	if (const TSharedPtr<FUICommandList> PinnedCommands = Commands.Pin())
	{
		return PinnedCommands->ProcessCommandBindings(KeyEvent);
	}
	else
	{
		return false;
	}
}


#undef LOCTEXT_NAMESPACE
