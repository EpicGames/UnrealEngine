// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once


#include "Framework/Application/IInputProcessor.h"
#include "Layout/WidgetPath.h"
#include "Templates/SharedPointer.h"
#include "Framework/Commands/UICommandList.h"


class SWidget;


//
// FAIAssistantInputProcessor
//
// NOTE_AI_ASSISTANT_INPUT_PROCESSOR
// The module's commands store the actual key events we would like to handle. And those commands can largely work on their own, without this Slate
// Input Processor. But, they can end up getting subjected to captured focus, so don't always work globally. So this Slate Input Processor is still
// necessary, since it always has global focus. It will simply pass the key event to the commands. 
//


class FAIAssistantInputProcessor : public IInputProcessor
{
public:


	explicit FAIAssistantInputProcessor(const TSharedPtr<FUICommandList>& InCommands);
	
	virtual void Tick(const float DeltaTime, FSlateApplication& SlateApp, TSharedRef<ICursor> Cursor) override; // ..required
	virtual bool HandleKeyDownEvent(FSlateApplication& SlateApp, const FKeyEvent& KeyEvent) override;
	

private:


	TWeakPtr<FUICommandList> Commands;
};