// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include <type_traits>

#include "Containers/UnrealString.h"
#include "Misc/TVariantMeta.h"
#include "Serialization/JsonSerializerBase.h"

#include "AIAssistantLog.h"

// Macro for TJsonVariantSerializer that provide a similar interface to JSON_SERIALIZE_VARIANT_*()
// This must be used within a BEGIN_JSON_SERIALIZER() block and must end with
// UE_JSON_SERIALIZE_ENUM_VARIANT_END().
// EnumTypeFieldName is the name of the JSON field that will contain the type string, EnumTypeField
// is the corresponding C++ field in the FJsonSerializable. VariantFieldName is the name of the
// JSON field that will be parsed into / instantiate a variant in the field VariantField.
// After this macro, UE_JSON_SERIALIZE_ENUM_VARIANT() is used to instantiate or parse variant
// types.
#define UE_JSON_SERIALIZE_ENUM_VARIANT_BEGIN( \
		EnumTypeFieldName, EnumTypeField, VariantFieldName, VariantField) \
	{ \
		UE::AIAssistant::TJsonVariantSerializer VariantSerializer( \
			TEXT(EnumTypeFieldName), EnumTypeField, TEXT(VariantFieldName), VariantField, \
			Serializer)

// Ends a block started with UE_JSON_SERIALIZE_ENUM_VARIANT_BEGIN().
#define UE_JSON_SERIALIZE_ENUM_VARIANT_END() \
	}

// Parse or instantiate the variant type VariantType when EnumTypeFieldName
// (see UE_JSON_SERIALIZE_ENUM_VARIANT_BEGIN) matches EnumTypeValue.
#define UE_JSON_SERIALIZE_ENUM_VARIANT(EnumTypeValue, VariantType) \
	VariantSerializer.Serialize<VariantType>(EnumTypeValue)


namespace UE::AIAssistant
{
	// Handles serializing a variant where the variant object's type is described by an enum field.
	// The enum, enum field name, variant type and variant field name can all be customized.
	// 
	// See AIAssistantJsonVariantSerializerTest.cpp for an example.
	template<typename VariantTypeEnumT, typename VariantT>
	class TJsonVariantSerializer
	{
	public:
		TJsonVariantSerializer(
			const TCHAR* VariantTypeFieldNameToSerialize,
			VariantTypeEnumT& VariantTypeFieldToSerialize,
			const TCHAR* VariantFieldNameToSerialize,
			VariantT& VariantToSerialize,
			FJsonSerializerBase& CurrentSerializer) :
			VariantTypeFieldName(VariantTypeFieldNameToSerialize),
			VariantTypeField(VariantTypeFieldToSerialize),
			VariantFieldName(VariantFieldNameToSerialize),
			VariantField(VariantToSerialize),
			Serializer(CurrentSerializer)
		{
			static_assert(std::is_enum_v<VariantTypeEnumT>);
			static_assert(TIsVariant_V<VariantT>);
			if (Serializer.IsLoading())
			{
				FString VariantTypeJson;
				Serializer.Serialize(VariantTypeFieldName, VariantTypeJson);
				LexFromString(VariantTypeField, *VariantTypeJson);
			}
		}

		TJsonVariantSerializer(const TJsonVariantSerializer&) = delete;
		TJsonVariantSerializer& operator=(const TJsonVariantSerializer&) = delete;

		// Call this once for each supported variant type and the associated enum value that instances
		// the type.
		template<typename T>
		void Serialize(VariantTypeEnumT VariantTypeEnumValueToMatch)
		{
			if (Serializer.IsLoading())
			{
				if (VariantTypeEnumValueToMatch == VariantTypeField)
				{
					auto VariantJsonObj = Serializer.GetObject()->GetObjectField(VariantFieldName);
					if (VariantJsonObj.IsValid())
					{
						VariantField.template Emplace<T>();
						VariantField.template Get<T>().FromJson(VariantJsonObj);
					}
					else
					{
						UE_LOGF(LogAIAssistant, Warning,
							"Failed to load variant from field '%ls' as it is missing.",
							VariantFieldName);
					}
				}
			}
			else
			{
				if (VariantField.template IsType<T>())
				{
					FString VariantTypeJson = LexToString(VariantTypeField);
					Serializer.Serialize(VariantTypeFieldName, VariantTypeJson);
					Serializer.StartObject(VariantFieldName);
					VariantField.template Get<T>().Serialize(Serializer, true);
					Serializer.EndObject();
				}
			}
		}

	private:
		const TCHAR* VariantTypeFieldName;
		VariantTypeEnumT& VariantTypeField;
		const TCHAR* VariantFieldName;
		VariantT& VariantField;
		FJsonSerializerBase& Serializer;
	};
}