// Copyright Epic Games, Inc. All Rights Reserved.

#include "AIAssistantLog.h"

#include "Logging/LogMacros.h"

DEFINE_LOG_CATEGORY(LogAIAssistant);