// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Future.h"
#include "Containers/Array.h"
#include "Templates/SharedPointer.h"

namespace UE::AIAssistant
{
	// Runs a series of asynchronous operations sequentially.
	//
	// Given a list of arguments and a predicate that returns TFuture<void>,
	// this function executes the predicate for each argument in order and
	// signals completion when all have finished.
	//
	// Index and InPromise are internal parameters used for recursive chaining.
	template <typename FArgument, typename FPredicate>
	TFuture<void> RunSequential(
		TArray<FArgument> Arguments,
		FPredicate Predicate,
		int32 Index = 0,
		TSharedPtr<TPromise<void>> InPromise = nullptr)
	{
		bool bIsTopLevel = !InPromise;
		auto Promise = bIsTopLevel ? MakeShared<TPromise<void>>() : InPromise;

		if (Index >= Arguments.Num())
		{
			Promise->SetValue();
			return bIsTopLevel ? Promise->GetFuture() : TFuture<void>();
		}

		Predicate(Arguments[Index]).Next(
			[Promise, Arguments = MoveTemp(Arguments), Predicate, Index]() mutable
			{
				RunSequential(MoveTemp(Arguments), Predicate, Index + 1, Promise);
			});

		// Only retrieve the future on the initial call. Recursive calls
		// discard the return value so they do not need a valid future.
		TFuture<void> ResultFuture;
		if (bIsTopLevel)
		{
			ResultFuture = Promise->GetFuture();
		}
		return ResultFuture;
	}
}
