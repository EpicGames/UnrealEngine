// Copyright Epic Games, Inc. All Rights Reserved.

#include "AIAssistantSlateQuerier.h"

#include "EditorModes.h"
#include "Containers/Set.h"
#include "Containers/ArrayView.h"
#include "LevelEditorSubsystem.h"
#include "LevelEditorViewport.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "EditorModeManager.h"
#include "EngineUtils.h"
#include "IDetailsView.h"
#include "Layout/WidgetPath.h"
#include "Interfaces/IMainFrameModule.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Culture.h"
#include "Serialization/JsonSerializable.h"
#include "Serialization/JsonSerializerMacros.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Subsystems/EditorActorSubsystem.h"
#include "Toolkits/BaseToolkit.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Docking/SDockTab.h"
#include "Filters/SFilterSearchBox.h"
#include "SGraphNode.h"
#include "Framework/Application/SlateApplication.h"

#include "AIAssistant.h"
#include "AIAssistantLog.h"
#include "AIAssistantSlateQuerierUtils.h"
#include "AIAssistantSlateQuerierUtilsTypeSearch.h"
#include "AIAssistantSubsystem.h"
#include "AIAssistantWebBrowser.h"


#define LOCTEXT_NAMESPACE "FAIAssistantSlateQuerier"

using namespace UE::AIAssistant::SlateQuerier;

//
// UE::AIAssistant::SlateQuerier
//
FWidgetPath UE::AIAssistant::SlateQuerier::GetWidgetPathUnderCursor()
{
	const FVector2f CursorPos = FSlateApplication::Get().GetCursorPos();
	const TArray<TSharedRef<SWindow>>& Windows = FSlateApplication::Get().GetTopLevelWindows();
	return FSlateApplication::Get().LocateWindowUnderMouse(CursorPos, Windows, true);
}

void UE::AIAssistant::SlateQuerier::QueryAIAssistantAboutSlateWidget(const FWidgetPath& InWidgetPath)
{
	if (!InWidgetPath.IsValid())
	{
		UE_LOGF(LogAIAssistant, Warning, "No valid widget path found to generate widget query.");
		return;
	}

	FSlateQuerier SlateQuerier = FSlateQuerier(InWidgetPath);
	if (!SlateQuerier.GenerateAndSendQuery())
	{
		UE_LOGF(LogAIAssistant, Warning, "Could not generate query for widget.");
	}
}

FSlateQuerier::FSlateQuerier(const FWidgetPath& InWidgetPath) 
	: OriginalWidgetPath(InWidgetPath), WidgetPath(InWidgetPath), bIsUIWidget(false), bIsObject(false),
	  bIsInAssistantPanel(false), GeneratedQuery(), StructuredContext()
{}

bool FSlateQuerier::GenerateAndSendQuery()
{
	IdentifyObjectOfQuery();
	GenerateQuery();
	if (IsValid())
	{
		GenerateStructuredContext();
		SendQuery();
	}
	return IsValid();
}

void FSlateQuerier::IdentifyObjectOfQuery()
{
	// For menu items, we want to generate the context based on the root widget that spawned the menu,
	// not on the actual menu item itself.
	if (Utility::FindClosestMenuItem(OriginalWidgetPath))
	{
		TSharedPtr<SWidget> RootWidget = FSlateApplication::Get().GetMenuHostWidget();
		if (RootWidget.IsValid())
		{
			FSlateApplication::Get().GeneratePathToWidgetChecked(RootWidget.ToSharedRef(), WidgetPath);
		}
	}

	if (!WidgetPath.IsValid())
	{
		return;
	}

	// Is there a current tool-tip?
	CurrentToolTipText = Utility::FindCurrentToolTipText();

	// Is there an identifiable asset editor, mode, or window?
	FindEditorName();

	// Is there an identifiable tab?
	FindTabName();

	// Is there text under the cursor??
	TextUnderCursor = Utility::FindTextUnderCursor(WidgetPath);

	// Is there an identifiable item?
	FindItemName();
}

void FSlateQuerier::GenerateQuery()
{
	FText FormattedItemName;
	if (!ItemName.IsEmpty())
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("ItemName"), ItemName);
		Args.Add(TEXT("ItemDescriptor"), ItemDescriptor);
		FormattedItemName = FText::Format(LOCTEXT("ItemName_Format", " the \"{ItemName}\" {ItemDescriptor}"), Args);
	}

	FText NameToQuery;
	if (bIsInAssistantPanel)
	{
		NameToQuery = LOCTEXT("ItemName_Assistant", "the AI Assistant");
	}
	else if (FormattedItemName.IsEmpty() && !TextUnderCursor.IsEmpty())
	{
		NameToQuery = TextUnderCursor;
	}
	else if (!FormattedItemName.IsEmpty() || !FormattedTabName.IsEmpty() || !FormattedWindowName.IsEmpty())
	{
		NameToQuery = (!FormattedItemName.IsEmpty()) ? FormattedItemName : (!FormattedTabName.IsEmpty()) ? FormattedTabName : FormattedWindowName;
	}

	// Localized text is non-static const so it can change when the editor language is changed.
	const FText QueryFormatWidget = LOCTEXT("QueryFormat1", "I would like to know what {NameToQuery} does.");
	const FText QueryFormatGeneral = LOCTEXT("QueryFormat2", "I would like to know what \"{NameToQuery}\" means.");
	const FText QueryFormatObject = LOCTEXT("QueryFormat3", "I would like to know about {NameToQuery}.");
	const FText QueryFormatAssistant = LOCTEXT("QueryFormat4", "Tell me about yourself.");
	// Hidden instruction text sent to assistant can be non-localized and static const.
	static const FText QueryInstructionsWidgetFormat = INVTEXT(
		"Provide an easily readable, informative, accurate answer that describes what the widget does in the Unreal Editor UI. {QueryAdditionalInstructions} {StyleInstructions}");
	static const FText QueryInstructionsObjectFormat = INVTEXT(
		"Provide an easily readable, informative, accurate answer that describes what I should know about it in the Unreal Editor UI. {QueryAdditionalInstructions} {StyleInstructions}");
	static const FText QueryInstructionsAssistantFormat = INVTEXT(
		"Provide an easily readable, informative, accurate answer that describes what I should know about yourself and what you can do for me. {StyleInstructions}");
	static const FText QueryAdditionalInstructionsWidget = INVTEXT(
		"Explain what happens if I select or click on it.");
	static const FText QueryAdditionalInstructionsObject = INVTEXT(
		"Describe what the object is or what it does.");
	static const FText QueryAdditionalInstructionsGeneral = INVTEXT(
		"If the object is clickable, like a button, then explain what happens if I click on it. If the object is something you select, like an asset or a blueprint node, describe what the object is or what it does.");
	static const FText StyleInstructions = INVTEXT(
		"Use bold text to emphasize Unreal Engine specific terms. Start with a quick overview paragraph, then add key points limited to one or two formatted sections with headers and bullet points. Don't include a summary at the end.");

	if (!NameToQuery.IsEmpty())
	{
		FFormatNamedArguments QueryArgs;
		QueryArgs.Add(TEXT("NameToQuery"), NameToQuery);
		FFormatNamedArguments InstructionArgs;

		if (bIsInAssistantPanel)
		{
			GeneratedQuery = FText::Format(QueryFormatAssistant, QueryArgs);
			InstructionArgs.Add(TEXT("StyleInstructions"), StyleInstructions);
			GeneratedQueryInstructions = FText::Format(QueryInstructionsAssistantFormat, InstructionArgs);
		}
		else if (bIsObject)
		{
			GeneratedQuery = FText::Format(QueryFormatObject, QueryArgs);
			InstructionArgs.Add(TEXT("QueryAdditionalInstructions"), QueryAdditionalInstructionsObject);
			InstructionArgs.Add(TEXT("StyleInstructions"), StyleInstructions);
			GeneratedQueryInstructions = FText::Format(QueryInstructionsObjectFormat, InstructionArgs);
		}
		else if (bIsUIWidget)
		{
			GeneratedQuery = FText::Format(QueryFormatWidget, QueryArgs);
			InstructionArgs.Add(TEXT("QueryAdditionalInstructions"), QueryAdditionalInstructionsWidget);
			InstructionArgs.Add(TEXT("StyleInstructions"), StyleInstructions);
			GeneratedQueryInstructions = FText::Format(QueryInstructionsWidgetFormat, InstructionArgs);
		}
		else
		{
			GeneratedQuery = FText::Format(QueryFormatWidget, QueryArgs);
			InstructionArgs.Add(TEXT("QueryAdditionalInstructions"), QueryAdditionalInstructionsGeneral);
			InstructionArgs.Add(TEXT("StyleInstructions"), StyleInstructions);
			GeneratedQueryInstructions = FText::Format(QueryInstructionsWidgetFormat, InstructionArgs);
		}
	}
}

void FSlateQuerier::GenerateStructuredContext()
{
	StructuredContext.UnrealEditorVersion = FEngineVersion::Current().ToString();
	StructuredContext.UnrealEditorLanguage = FInternationalization::Get().GetCurrentLanguage()->GetDisplayName();
	StructuredContext.UnrealEditorLocale = FInternationalization::Get().GetCurrentLocale()->GetName();

	static const FText LanguageInstructionsFormat = INVTEXT(" Please respond in language \"{0}\". ");
	const FText Language = FText::FromString(StructuredContext.UnrealEditorLanguage);
	LanguageInstructions = FText::Format(LanguageInstructionsFormat, Language);

	StructuredContext.ToolTipText = CurrentToolTipText.ToString();

	if (!StructuredContext.ActorClass || StructuredContext.ActorClass->IsEmpty())
	{
		// If we're in a Details panel, include the class of actor or object that we're editing.
		const TSharedPtr<SWidget> DetailsView = Utility::FindClosestWidgetPtrOfType(WidgetPath, "SDetailsView");
		if (DetailsView.IsValid())
		{
			const TSharedPtr<IDetailsView> DetailsViewCast = StaticCastSharedPtr<IDetailsView>(DetailsView);
			if (DetailsViewCast->GetSelectedActors().Num() > 0)
			{
				FString ActorName;
				DetailsViewCast->GetSelectedActors()[0]->GetClass()->GetName(ActorName);
				StructuredContext.ActorClass = ActorName;
			}
			else if (DetailsViewCast->GetSelectedObjects().Num() > 0)
			{
				FString ObjectName;
				DetailsViewCast->GetSelectedObjects()[0]->GetClass()->GetName(ObjectName);
				StructuredContext.ActorClass = ObjectName;
			}
		}
	}

	if (!ItemName.IsEmpty())
	{
		StructuredContext.ItemName = ItemName.ToString();
		StructuredContext.ItemDescriptor = ItemDescriptor.ToString();
	}

	if (!TextUnderCursor.IsEmpty())
	{
		StructuredContext.TextUnderCursor = TextUnderCursor.ToString();
	}

	// Get top level window title.
	TSharedRef<SWindow> WidgetWindow = WidgetPath.GetWindow();
	FText WindowTitle = WidgetWindow->GetTitle();
	if (!WindowTitle.IsEmpty())
	{
		StructuredContext.TopWindowName = WindowTitle.ToString();
	}

	// Get picked widget type.
	if (LastPickedWidget.IsValid())
	{
		StructuredContext.WidgetType = LastPickedWidget->GetType().ToString();
	}

	if (ItemName.IsEmpty() && !bIsInAssistantPanel)
	{
		StructuredContext.WidgetPath = TArray<FString>();
		// Add simplified widget path
		for (int32 WidgetIndex = 0; WidgetIndex < WidgetPath.Widgets.Num(); WidgetIndex++)
		{
			const FArrangedWidget* ArrangedWidget = &WidgetPath.Widgets[WidgetIndex];
			const TSharedRef<SWidget>& ThisWidget = ArrangedWidget->Widget;
			FString NextWidgetString = ThisWidget->GetType().ToString();
			// Skip uninteresting widgets
			if (NextWidgetString != TEXT("SBorder") && NextWidgetString != TEXT("SBox") &&
				NextWidgetString != TEXT("SScrollBorder") &&
				NextWidgetString != TEXT("SOverlay") && NextWidgetString != TEXT("SVerticalBox") &&
				NextWidgetString != TEXT("SHorizontalBox") && NextWidgetString != TEXT("SSplitter") &&
				NextWidgetString != TEXT("SDockingSplitter"))
			{
				StructuredContext.WidgetPath->Add(NextWidgetString);
			}
		}
	}
}

void FSlateQuerier::SendQuery()
{
	TSharedPtr<SAIAssistantWebBrowser> WebBrowser =
		UAIAssistantSubsystem::GetAIAssistantWebBrowserWidget();
	if (!WebBrowser.IsValid())
	{
		UE_LOGF(
			LogAIAssistant, Warning,
			"Cannot send query: AI Assistant web browser widget is not available.");
		return;
	}
	WebBrowser->CreateConversation();

	const FString VisiblePromptString =
		Utility::CleanExtraWhiteSpaceFromString(GeneratedQuery.ToString());

	const FString HiddenContextString =
		Utility::CleanExtraWhiteSpaceFromString(GeneratedQueryInstructions.ToString()) +
		LanguageInstructions.ToString() +
		TEXT(" ContextData=") +
		StructuredContext.ToJson(false);

	WebBrowser->AddUserMessageToConversation(VisiblePromptString, HiddenContextString);
}

bool FSlateQuerier::FindEditorName()
{
	FText OutName = FText();

	if (!WidgetPath.IsValid())
	{
		FormattedWindowName = OutName;
		return false;
	}

	// is it in the status bar?
	TSharedPtr<SWidget> StatusBar = Utility::FindClosestWidgetPtrOfType(WidgetPath, "SStatusBar");
	if (StatusBar.IsValid())
	{
		LastPickedWidget = StatusBar;
		StructuredContext.TabName = TEXT("StatusBar");
		FormattedWindowName = LOCTEXT("TabName_StatusBar", " the Status Bar");
		return true;
	}

	if (IsInEditor())
	{
		return true;
	}

	// Is it in a drawer overlay?
	if (IsInDrawer())
	{
		return true;
	}

	TSharedRef<SWindow> WidgetWindow = WidgetPath.GetWindow();
	// is it the main window that hosts the level editor?
	if (FModuleManager::Get().IsModuleLoaded("LevelEditor"))
	{
		IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>(TEXT("MainFrame"));
		if (WidgetWindow.ToSharedPtr() == MainFrameModule.GetParentWindow())
		{
			StructuredContext.WindowName = TEXT("MainFrame");
			FormattedWindowName = LOCTEXT("WindowMainFrame_Title", " the editor's main window");
			return true;
		}
	}

	// does the window have a title, like the Project Browser or Color Picker?
	FText WindowTitle = WidgetWindow->GetTitle();
	if (!WindowTitle.IsEmpty())
	{
		StructuredContext.WindowName = WindowTitle.ToString();
		LastPickedWidget = WidgetWindow.ToSharedPtr();
		FFormatNamedArguments Args;
		Args.Add(TEXT("WindowTitle"), WindowTitle);
		FormattedWindowName = FText::Format(LOCTEXT("WindowNameFormat_Title", " the {WindowTitle} window"), Args);
		return true;
	}

	// TODO: how to find other editor contexts?
	return false;
}

bool FSlateQuerier::FindTabName()
{
	bool bFoundTabName = false;

	TSharedPtr<SWidget> TabStack = Utility::FindClosestWidgetPtrOfType(WidgetPath, "SDockingTabStack");
	if (TabStack.IsValid())
	{
		TArray<TSharedRef<SWidget>> DockTabs;
		Utility::FindChildWidgetsOfType(DockTabs, TabStack, "SDockTab");
		for (auto& ThisTab : DockTabs)
		{
			TSharedRef<SDockTab> ThisTabCast = StaticCastSharedRef<SDockTab>(ThisTab);
			if (ThisTabCast->GetTabRole() == ETabRole::MajorTab || ThisTabCast->GetTabRole() == ETabRole::DocumentTab)
			{
				break;
			}
			if (ThisTabCast->IsForeground())
			{
				LastPickedWidget = TabStack;
				StructuredContext.TabName = ThisTabCast->GetTabLabel().ToString();
				FFormatNamedArguments Args;
				Args.Add(TEXT("PanelName"), ThisTabCast->GetTabLabel());
				FormattedTabName = FText::Format(LOCTEXT("TabName_Format", " the {PanelName} panel"), Args);
				bFoundTabName = true;
			}
		}
	}

	return bFoundTabName;
}

bool FSlateQuerier::IsInEditor()
{
	static const TSet<FEditorModeID> BlockedModeIDs = TSet<FEditorModeID>({
		FBuiltinEditorModes::EM_Default,
		"EditMode.SubTrackEditMode"
		});

	// are we in the level editor?
	TSharedPtr<SWidget> LevelEditor = Utility::FindFirstWidgetPtrOfType(WidgetPath, "SLevelEditor");
	if (LevelEditor)
	{
		StructuredContext.EditorName = TEXT("LevelEditor");
		FText ModeString;
		ULevelEditorSubsystem* LevelEditorSubsystem = GEditor->GetEditorSubsystem<ULevelEditorSubsystem>();
		if (FEditorModeTools* ModeManager = LevelEditorSubsystem->GetLevelEditorModeManager())
		{
			UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
			TArray<FEditorModeInfo> ModeInfos = AssetEditorSubsystem->GetEditorModeInfoOrderedByPriority();
			for (const FEditorModeInfo& ModeInfo : ModeInfos)
			{
				if (UEdMode* EdMode = ModeManager->GetActiveScriptableMode(ModeInfo.ID))
				{
					TWeakPtr<FModeToolkit> ModeToolkit = EdMode->GetToolkit();
					if (ModeToolkit.IsValid())
					{
						FText ToolDisplayName = ModeToolkit.Pin()->GetActiveToolDisplayName();
						if (!ToolDisplayName.IsEmpty())
						{
							StructuredContext.ActiveToolName = ToolDisplayName.ToString();
						}
					}
				}
				if (ModeString.IsEmpty() && ModeManager->IsModeActive(ModeInfo.ID) && !BlockedModeIDs.Contains(ModeInfo.ID))
				{
					FFormatNamedArguments Args;
					Args.Add(TEXT("ModeName"), ModeInfo.Name);
					ModeString = FText::Format(LOCTEXT("EditorName_ModeFormat", "with {ModeName} mode active"), Args);
					StructuredContext.EditorMode = ModeInfo.Name.ToString();
				}
			}
		}
		LastPickedWidget = LevelEditor;
		if (ModeString.IsEmpty())
		{
			FormattedWindowName = LOCTEXT("EditorName_LevelEditor", " the Level Editor");
		}
		else
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("WithModeActive"), ModeString);
			FormattedWindowName = FText::Format(LOCTEXT("EditorName_LevelEditorAndMode", " the Level Editor {WithModeActive}"), Args);
		}

		if (!GEditor->IsPlaySessionInProgress())
		{
			// Get selected actors, if any.
			bool bActorIsSelected = false;
			UEditorActorSubsystem* EditorActorSubsystem = GEditor->GetEditorSubsystem<UEditorActorSubsystem>();
			// Get selected actors
			TArray<AActor*> SelectedActors = EditorActorSubsystem->GetSelectedLevelActors();
			for (AActor* Actor : SelectedActors)
			{
				if (Actor)
				{
					SelectedActorName = Actor->GetName();
					SelectedActorClass = Actor->GetClass()->GetName();
					bActorIsSelected = true;
					break;
				}
			}

			// If there is no selection and mouse is in viewport, see what actor is under mouse cursor.
			if (!bActorIsSelected && Utility::FindClosestWidgetPtrOfType(WidgetPath, "SViewport"))
			{
				FViewport* ActiveViewport = GEditor->GetActiveViewport();
				if (ActiveViewport)
				{
					FIntPoint MousePos;
					ActiveViewport->GetMousePos(MousePos);
					HHitProxy* HitProxy = ActiveViewport->GetHitProxy(MousePos.X, MousePos.Y);
					if (HitProxy && HitProxy->IsA(HActor::StaticGetType()))
					{
						HActor* TargetProxy = static_cast<HActor*>(HitProxy);
						AActor* TargetActor = TargetProxy->Actor;
						if (TargetActor)
						{
							SelectedActorName = TargetActor->GetName();
							SelectedActorClass = TargetActor->GetClass()->GetName();
							bActorIsSelected = true;
						}
					}
				}
			}
		}

		return true;
	}

	// if not, are we in a different asset editor?
	TSharedPtr<SWidget> StandaloneAssetEditorHost = Utility::FindFirstWidgetPtrOfType(WidgetPath, "SStandaloneAssetEditorToolkitHost");
	if (StandaloneAssetEditorHost)
	{
		TArray<TSharedRef<SWidget>> TabStacks;
		Utility::FindChildWidgetsOfType(TabStacks, StandaloneAssetEditorHost, "SDockingTabStack");
		for (auto& ThisTabStack : TabStacks)
		{
			TArray<TSharedRef<SWidget>> DockTabs;
			Utility::FindChildWidgetsOfType(DockTabs, ThisTabStack.ToSharedPtr(), "SDockTab");

			for (auto& ThisTab : DockTabs)
			{
				TSharedRef<SDockTab> ThisTabCast = StaticCastSharedRef<SDockTab>(ThisTab);
				if (ThisTabCast->IsForeground())
				{
					TSharedPtr<FTabManager> PickedWidgetTabManager = ThisTabCast->GetTabManagerPtr();

					if (PickedWidgetTabManager.IsValid())
					{
						UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
						TArray<UObject*> AllEditedAssets = AssetEditorSubsystem->GetAllEditedAssets();
						for (auto& ThisEditedAsset : AllEditedAssets)
						{
							IAssetEditorInstance* ThisEditor = AssetEditorSubsystem->FindEditorForAsset(ThisEditedAsset, false);
							if (ThisEditor && ThisEditor->GetAssociatedTabManager() == PickedWidgetTabManager)
							{
								StructuredContext.EditorName = ThisEditor->GetEditorName().ToString();
								LastPickedWidget = ThisTabStack.ToSharedPtr();
								FFormatNamedArguments Args;
								Args.Add(TEXT("AssetEditorName"), FText::FromName(ThisEditor->GetEditorName()));
								FormattedWindowName = FText::Format(
									LOCTEXT("EditorName_AssetEditorFormat", " the {AssetEditorName} editor"), Args);

								SelectedActorName = ThisEditedAsset->GetName();
								SelectedActorClass = ThisEditedAsset->GetClass()->GetName();

								return true;
							}
						}
					}
				}
			}
		}
	}
	return false;
}

bool FSlateQuerier::IsInDrawer()
{
	TSharedPtr<SWidget> DrawerOverlay = Utility::FindFirstWidgetPtrOfType(WidgetPath, "SDrawerOverlay");
	if (DrawerOverlay.IsValid())
	{
		FText DrawerName;
		TSharedPtr<SWidget> DrawerWidget = Utility::FindChildWidgetOfType(DrawerOverlay, "SContentBrowser");
		if (!DrawerWidget)
		{
			DrawerWidget = Utility::FindChildWidgetOfType(DrawerOverlay, "SOutputLog");
		}

		if (DrawerWidget)
		{
			if (DrawerWidget->GetType() == "SContentBrowser")
			{
				DrawerName = LOCTEXT("DrawerName_ContentBrowser", "ContentBrowser");
			}
			else if (DrawerWidget->GetType() == "SOutputLog")
			{
				DrawerName = LOCTEXT("DrawerName_OutputLog", "OutputLog");
			}
			// TODO: Are there other items that are commonly in drawers?

			if (!DrawerName.IsEmpty())
			{
				StructuredContext.WindowName = DrawerOverlay->GetType().ToString();
				StructuredContext.TabName = DrawerName.ToString();
				LastPickedWidget = DrawerWidget;
				FFormatNamedArguments Args;
				Args.Add(TEXT("DrawerName"), DrawerName);
				FormattedWindowName = FText::Format(LOCTEXT("DrawerNameFormat", "the {DrawerName} drawer"), Args);
				return true;
			}
		}
	}
	return false;
}

bool FSlateQuerier::FindItemName()
{
	ItemDescriptor = LOCTEXT("ItemDescriptor_Generic", "control");

	if (!WidgetPath.IsValid())
	{
		return false;
	}

	if (// Is it a graph node?
		ItemIsGraphNode() ||
		// Is it a search/filter field?
		ItemIsSearchBox() ||
		// Is it a console input box?
		ItemIsConsoleInputBox() ||
		// Is it a details panel property?
		ItemIsDetailsProperty() ||
		// Is it a plugin in the plugins browser ?
		ItemIsPlugin() ||
		// Is it a breadcrumb trail button?
		ItemIsBreadcrumbTrailButton() ||
		// Is it an asset item?
		ItemIsAsset() ||
		// Is it an Actor in Outliner, or an instance or component in the SubobjectInstanceEditor?
		ItemIsOutlinerActorOrSubobject() ||
		// Is it one of the Details panel checkboxes that filter the detail properties viewed?
		ItemIsDetailPanelFilterCheckbox() ||
		// Does the widget path end up in a Viewport?
		ItemIsInViewport() ||
		// Is it a menu item?
		ItemIsMenuItem() ||
		// Is it a button?
		ItemIsButton())
	{
		return true;
	}

	return false;
}

bool FSlateQuerier::ItemIsInViewport()
{
	TSharedPtr<SWidget> Viewport = Utility::FindClosestWidgetPtrOfType(WidgetPath, "SViewport");
	if (Viewport.IsValid())
	{
		// Web browser widgets also have Viewports!
		// Ignore web browser views except for the special case of AI Assistant window or panel.
		TSharedPtr<SWidget> WebBrowser = Viewport->GetParentWidget();
		if (WebBrowser.IsValid() && WebBrowser->GetType() == "SWebBrowserView")
		{
			TSharedPtr<SWidget> AssistantPane = Utility::FindClosestWidgetPtrOfType(WidgetPath, "SAIAssistantWebBrowser");
			if (AssistantPane.IsValid())
			{
				bIsInAssistantPanel = true;
				LastPickedWidget = AssistantPane;
				return true;
			}
		}
		else
		{
			// Selected actors are filled in by the IsInEditor function.
			if (!SelectedActorName.IsEmpty())
			{
				bIsObject = true;
				LastPickedWidget = Viewport;
				ItemName = FText::FromString(SelectedActorName);
				ItemDescriptor = LOCTEXT("ItemDescriptor_SelectedActor", "actor");
				StructuredContext.ActorClass = SelectedActorClass;
				return true;
			}
		}
	}
	return false;
}

bool FSlateQuerier::ItemIsGraphNode()
{
	// Look up for an SGraphEditor and then down for an SGraphPanel. Whatever is the child of that SGraphPanel is our SGraphNode.
	// Note that this rests on the assumption that SGraphPanel only contains children that are SGraphNodes. This assumption is also
	// made in the code of SGraphPanel itself.
	TSharedPtr<SGraphEditor> GraphEditor;
	for (int32 WidgetIndex = WidgetPath.Widgets.Num() - 1; WidgetIndex >= 0; --WidgetIndex)
	{
		const FArrangedWidget* ArrangedWidget = &WidgetPath.Widgets[WidgetIndex];
		const TSharedRef<SWidget>& ThisWidget = ArrangedWidget->Widget;
		if (ThisWidget->GetType() == "SGraphEditor")
		{
			for (int32 EditorDescendantIndex = WidgetIndex; EditorDescendantIndex < WidgetPath.Widgets.Num(); EditorDescendantIndex++)
			{
				const FArrangedWidget* DescendantArrangedWidget = &WidgetPath.Widgets[EditorDescendantIndex];
				const TSharedRef<SWidget>& ThisDescendantWidget = DescendantArrangedWidget->Widget;
				if (ThisDescendantWidget->GetType() == "SGraphPanel" && (EditorDescendantIndex + 1 < WidgetPath.Widgets.Num()))
				{
					const FArrangedWidget* NodeArrangedWidget = &WidgetPath.Widgets[EditorDescendantIndex+1];
					const TSharedRef<SWidget>& ThisNodeWidget = NodeArrangedWidget->Widget;
					const TSharedPtr<SGraphNode> AsGraphNode = StaticCastSharedPtr<SGraphNode>(ThisNodeWidget.ToSharedPtr());
					LastPickedWidget = ThisNodeWidget.ToSharedPtr();
					ItemName = AsGraphNode->GetNodeObj()->GetNodeTitle(ENodeTitleType::MenuTitle);
					ItemDescriptor = LOCTEXT("ItemDescriptor_GraphNode", "graph node");
					return true;
				}
			}
		}
	}
	return false;
}

bool FSlateQuerier::ItemIsSearchBox()
{
	static const TSet<FName> SearchBoxTypes = TSet<FName>({"SSearchBox", "SFilterSearchBox"});
	TSharedPtr<SWidget> SearchBox = Utility::FindClosestWidgetPtrOfTypes(WidgetPath, SearchBoxTypes);

	if (SearchBox.IsValid())
	{
		FText HintText;
		if (SearchBox->GetType() == "SSearchBox")
		{
			TSharedPtr<SSearchBox> ThisSearchBoxCast = StaticCastSharedPtr<SSearchBox>(SearchBox);
			HintText = ThisSearchBoxCast->GetHintText();
		}
		else
		{
			TSharedPtr<SWidget> EditableText = Utility::FindChildWidgetOfType(SearchBox, "SEditableText");
			if (EditableText)
			{
				TSharedPtr<SEditableText> ThisEditableTextCast = StaticCastSharedPtr<SEditableText>(EditableText);
				HintText = ThisEditableTextCast->GetHintText();
			}
		}
		ItemName = FText::Format(LOCTEXT("SearchBoxFormat", "{0}"),
			(HintText.IsEmpty()) ? LOCTEXT("SearchBoxDefaultName", "default") : HintText);
		ItemDescriptor = LOCTEXT("ItemDescriptor_SearchBox", "search box");
		LastPickedWidget = SearchBox;
		bIsUIWidget = true;
		return true;
	}

	return false;
}

bool FSlateQuerier::ItemIsConsoleInputBox()
{
	TSharedPtr<SWidget> ConsoleInputBox = Utility::FindClosestWidgetPtrOfType(WidgetPath, "SConsoleInputBox");
	if (ConsoleInputBox.IsValid())
	{
		FText HintText;
		TSharedPtr<SWidget> EditableText = Utility::FindChildWidgetOfType(ConsoleInputBox, "SMultiLineEditableTextBox");
		if (EditableText.IsValid())
		{
			TSharedPtr<SMultiLineEditableTextBox> ThisMultiLineEditableTextBoxCast = StaticCastSharedPtr<SMultiLineEditableTextBox>(EditableText);
			HintText = ThisMultiLineEditableTextBoxCast->GetHintText();
		}
		ItemName = FText::Format(LOCTEXT("ConsoleInputBoxFormat", "{0}"),
			(HintText.IsEmpty() ? LOCTEXT("ConsoleInputBoxDefaultName","Console") : HintText));
		ItemDescriptor = LOCTEXT("ItemDescriptor_ConsoleInputBox", "input box");
		LastPickedWidget = ConsoleInputBox;
		bIsUIWidget = true;
		return true;
	}
	return false;
}

bool FSlateQuerier::ItemIsDetailsProperty()
{
	// Look up for an SDetailSingleItemRow [could be others?] then down through the SPropertyNameWidget.
	TSharedPtr<SWidget> PropertyRow = Utility::FindClosestWidgetPtrOfType(WidgetPath, "SDetailSingleItemRow");
	if (PropertyRow.IsValid())
	{
		FText ChildText;

		LastPickedWidget = PropertyRow;

		TSharedPtr<SWidget> PropertyNameWidget = Utility::FindChildWidgetOfType(PropertyRow, "SPropertyNameWidget");
		if (PropertyNameWidget.IsValid())
		{
			ChildText = Utility::FindChildWidgetWithText(PropertyNameWidget);
			if (!ChildText.IsEmpty())
			{
				ItemName = ChildText;
				ItemDescriptor = LOCTEXT("ItemDescriptor_Property", "setting");
			}
		}
		else
		{
			// Some rows e.g. Transform have no SPropertyNameWidget. Best we can do is look for a text label.
			ChildText = Utility::FindChildWidgetWithText(PropertyRow);
			if (!ChildText.IsEmpty())
			{
				ItemName = ChildText;
				ItemDescriptor = LOCTEXT("ItemDescriptor_Property", "setting");
			}
		}

		// Find section name from sibling SDetailCategoryTableRow.
		if (PropertyRow->IsParentValid())
		{
			TSharedPtr<SWidget> ListPanel = PropertyRow->GetParentWidget();
			TSharedRef<SWidget> PreviousDetailCategory = SNullWidget::NullWidget;
			for (int32 ChildIdx = 0; ChildIdx < ListPanel->GetChildren()->Num(); ChildIdx++)
			{
				TSharedRef<SWidget> ThisWidget = ListPanel->GetChildren()->GetChildAt(ChildIdx);
				if (ThisWidget == PropertyRow)
				{
					break;
				}
				if (ThisWidget->GetType() == "SDetailCategoryTableRow")
				{
					PreviousDetailCategory = ThisWidget;
				}
			}
			if (PreviousDetailCategory->GetType() != "SNullWidgetContent")
			{
				ChildText = Utility::FindChildWidgetWithText(PreviousDetailCategory);
				if (!ChildText.IsEmpty())
				{
					StructuredContext.CategoryName = ChildText.ToString();
					ItemDescriptor = FText::Format(
						LOCTEXT("ItemDescriptor_PropertyWithCategory", "setting in the {0} category"), ChildText);
				}
			}
		}
		return true;
	}
	return false;
}

bool FSlateQuerier::ItemIsPlugin()
{
	// Look up for containing SPluginBrowser.
	TSharedPtr<SWidget> PluginBrowser = Utility::FindClosestWidgetPtrOfType(WidgetPath, "SPluginBrowser");
	if (PluginBrowser.IsValid())
	{
		FText ChildText;
		TSharedPtr<SWidget> PluginCategoryTree = Utility::FindClosestWidgetPtrOfType(WidgetPath, "SPluginCategoryTree");
		TSharedPtr<SWidget> PluginTileList = Utility::FindClosestWidgetPtrOfType(WidgetPath, "SPluginTileList");
		if (PluginCategoryTree.IsValid())
		{
			LastPickedWidget = PluginCategoryTree;
			TSharedPtr<SWidget> PluginCategory = Utility::FindClosestWidgetPtrOfType(WidgetPath, "SPluginCategory");
			if (PluginCategory.IsValid())
			{
				LastPickedWidget = PluginCategory;
				ChildText = Utility::FindChildWidgetWithText(PluginCategory);
				if (!ChildText.IsEmpty())
				{
					ItemName = ChildText;
					ItemDescriptor = LOCTEXT("ItemDescriptor_PluginCategory", "plugin category");
					bIsObject = true;
					return true;
				}
			}
		}
		else if (PluginTileList.IsValid())
		{
			LastPickedWidget = PluginTileList;
			TSharedPtr<SWidget> PluginTile = Utility::FindClosestWidgetPtrOfType(WidgetPath, "SPluginTile");
			if (PluginTile.IsValid())
			{
				LastPickedWidget = PluginTile;
				ChildText = Utility::FindChildWidgetWithText(PluginTile);
				if (!ChildText.IsEmpty())
				{
					ItemName = ChildText;
					ItemDescriptor = LOCTEXT("ItemDescriptor_PluginTile", "plugin");
					bIsObject = true;
					return true;
				}
			}
		}
	}
	return false;
}

bool FSlateQuerier::ItemIsBreadcrumbTrailButton()
{
	TSharedPtr<SWidget> BreadcrumbTrail = Utility::FindClosestWidgetPtrOfType(WidgetPath, "SBreadcrumbTrail<FNavigationCrumb>");
	TSharedPtr<SWidget> BreadcrumbButton = BreadcrumbTrail ? Utility::FindClosestWidgetPtrOfType(WidgetPath, "SButton") : nullptr;
	FText ChildText = Utility::FindChildWidgetWithText(BreadcrumbButton);
	if (!ChildText.IsEmpty())
	{
		ItemName = ChildText;
		ItemDescriptor = LOCTEXT("ItemDescriptor_Breadcrumb", "navigation breadcrumb");
		LastPickedWidget = BreadcrumbButton;
		bIsUIWidget = true;
		return true;
	}
	return false;
}

bool FSlateQuerier::ItemIsAsset()
{
	FText ChildText;

	// Is it an asset tile item?
	TSharedPtr<SWidget> AssetTileItem = Utility::FindClosestWidgetPtrOfType(WidgetPath, "SAssetTileItem");
	if (AssetTileItem.IsValid())
	{
		TSharedPtr<SWidget> AssetThumbnail = Utility::FindChildWidgetOfType(AssetTileItem, "SAssetThumbnail");
		if (AssetThumbnail.IsValid())
		{
			ChildText = Utility::FindChildWidgetWithText(AssetThumbnail); // This returns TYPE of asset instead of name
			if (!ChildText.IsEmpty())
			{
				FFormatNamedArguments Args;
				Args.Add(TEXT("AssetType"), ChildText);
				ItemDescriptor = FText::Format(LOCTEXT("ItemDescriptor_Asset", "{AssetType} asset"), Args);
			}

			if (!CurrentToolTipText.IsEmpty())
			{
				// We can use the tooltip for the asset's name.
				ItemName = CurrentToolTipText;
			}
			LastPickedWidget = AssetThumbnail;
			bIsObject = true;
		}
		else
		{
			ChildText = Utility::FindChildWidgetWithText(AssetTileItem);
			if (!ChildText.IsEmpty())
			{
				ItemName = ChildText;
				ItemDescriptor = LOCTEXT("ItemDescriptor_Folder", "asset folder");
				LastPickedWidget = AssetTileItem;
				bIsObject = true;
			}
		}
		return true;
	}

	// Is it an asset tree item?
	TSharedPtr<SWidget> AssetTreeItem = Utility::FindClosestWidgetPtrOfType(WidgetPath, "SAssetTreeItem");
	if (AssetTreeItem.IsValid())
	{
		ChildText = Utility::FindChildWidgetWithText(AssetTreeItem);
		if (!ChildText.IsEmpty())
		{
			ItemName = ChildText;
			ItemDescriptor = LOCTEXT("ItemDescriptor_Folder", "asset folder");
			LastPickedWidget = AssetTreeItem;
			bIsObject = true;
			return true;
		}
	}
	return false;
}

bool FSlateQuerier::ItemIsOutlinerActorOrSubobject()
{
	FText ChildText;

	// Is it an actor in the Outliner?
	TSharedPtr<SWidget> Outliner = Utility::FindClosestWidgetPtrOfType(WidgetPath, "SSceneOutliner");
	TSharedPtr<SWidget> ActorTreeLabel = Outliner ? Utility::FindClosestWidgetPtrOfType(WidgetPath, "SActorTreeLabel") : nullptr;
	TSharedPtr<SWidget> OutlinerRow = Outliner ? Utility::FindClosestWidgetPtrOfType(WidgetPath, "SSceneOutlinerTreeRow") : nullptr;

	if (!ActorTreeLabel.IsValid() && OutlinerRow.IsValid())
	{
		ActorTreeLabel = Utility::FindChildWidgetOfType(OutlinerRow, "SActorTreeLabel");
	}

	if (ActorTreeLabel.IsValid())
	{
		ChildText = Utility::FindChildWidgetWithText(ActorTreeLabel);
		if (!ChildText.IsEmpty())
		{
			ItemName = ChildText;
			ItemDescriptor = LOCTEXT("ItemDescriptor_OutlinerActor", "actor");
			LastPickedWidget = ActorTreeLabel;
			bIsObject = true;
			return true;
		}
	}

	// Is it an object in the Subobject Instance editor?
	TSharedPtr<SWidget> SubobjectInstanceEditor = Utility::FindClosestWidgetPtrOfType(WidgetPath, "SSubobjectInstanceEditor");
	TSharedPtr<SWidget> SubObjectRow = SubobjectInstanceEditor ? Utility::FindClosestWidgetPtrOfType(WidgetPath, "SSubobject_RowWidget") : nullptr;
	TSharedPtr<SWidget> NameWidget = SubObjectRow ? Utility::FindChildWidgetOfType(SubObjectRow, "SInlineEditableTextBlock") : nullptr;

	if (NameWidget.IsValid())
	{
		ChildText = Utility::FindChildWidgetWithText(NameWidget);
		ItemName = ChildText;
		ItemDescriptor = LOCTEXT("ItemDescriptor_SubobjectInstance", "instance");
		LastPickedWidget = SubObjectRow;
		bIsObject = true;
		return true;
	}

	return false;
}

bool FSlateQuerier::ItemIsDetailPanelFilterCheckbox()
{
	TSharedPtr<SWidget> ActorDetails = Utility::FindClosestWidgetPtrOfType(WidgetPath, "SActorDetails");
	if (!ActorDetails)
	{
		return false;
	}

	TSharedPtr<SWidget> DetailsView = Utility::FindClosestWidgetPtrOfType(WidgetPath, "SDetailsView");
	TSharedPtr<SWidget> DetailsNameArea = Utility::FindClosestWidgetPtrOfType(WidgetPath, "SDetailsNameArea");
	TSharedPtr<SWidget> SubobjectInstanceEditor = Utility::FindClosestWidgetPtrOfType(WidgetPath, "SSubobjectInstanceEditor");
	TSharedPtr<SWidget> WrapBox = Utility::FindClosestWidgetPtrOfType(WidgetPath, "SWrapBox");
	TSharedPtr<SWidget> CheckBox = Utility::FindClosestWidgetPtrOfType(WidgetPath, "SCheckBox");

	if (!DetailsView && !DetailsNameArea && !SubobjectInstanceEditor && WrapBox && CheckBox)
	{
		return SetItemAsButton(CheckBox, LOCTEXT("ItemDescriptor_PropertyFilter", "property filter"));
	}
	return false;
}

bool FSlateQuerier::ItemIsMenuItem()
{
	// Look up for an SMenuEntryBlock or SWidgetBlock and then down for the text.
	TSharedPtr<SWidget> MenuItemBlock = Utility::FindClosestMenuItem(WidgetPath);
	if (MenuItemBlock.IsValid())
	{
		FText ChildText = Utility::FindChildWidgetWithText(MenuItemBlock);
		if (!ChildText.IsEmpty())
		{
			ItemName = ChildText;
			ItemDescriptor = LOCTEXT("ItemDescriptor_Menu", "menu item");
			LastPickedWidget = MenuItemBlock;
			bIsUIWidget = true;
			return true;
		}
	}
	return false;
}

bool FSlateQuerier::ItemIsButton()
{
	static const TSet<FName> ToolbarButtonTypes = TSet<FName>({
		"SToolBarButtonBlock", "SToolBarComboButtonBlock"
		});

	static const TSet<FName> SimpleButtonTypes = TSet<FName>({
			"SButton",
			"SPrimaryButton",
			"SCheckBox"
		});

	return (ItemIsButtonType(ToolbarButtonTypes, LOCTEXT("ItemDescriptor_ToolbarButton", "toolbar button")) ||
			ItemIsButtonType(SimpleButtonTypes, LOCTEXT("ItemDescriptor_Button", "button")));
}

bool FSlateQuerier::ItemIsButtonType(const TSet<FName> ButtonWidgetTypes, FText InItemDescriptor)
{
	TSharedPtr<SWidget> Button = Utility::FindClosestWidgetPtrOfTypes(WidgetPath, ButtonWidgetTypes);

	// Disabled buttons *contain* the menu entry block.
	if (!Button.IsValid() && WidgetPath.GetLastWidget()->GetChildren()->Num() > 0)
	{
		for (int32 ChildIdx = 0; ChildIdx < WidgetPath.GetLastWidget()->GetChildren()->Num(); ChildIdx++)
		{
			TSharedRef<SWidget> ThisWidget = WidgetPath.GetLastWidget()->GetChildren()->GetChildAt(ChildIdx);
			if (ButtonWidgetTypes.Contains(ThisWidget->GetType()))
			{
				Button = ThisWidget.ToSharedPtr();
			}
		}
	}

	return Button.IsValid() && SetItemAsButton(Button, InItemDescriptor);
}

bool FSlateQuerier::SetItemAsButton(const TSharedPtr<SWidget> Button, FText InItemDescriptor)
{
	FText ChildText = Utility::FindChildWidgetWithText(Button);
	if (!ChildText.IsEmpty() || !CurrentToolTipText.IsEmpty())
	{
		if (!ChildText.IsEmpty())
		{
			ItemName = ChildText;
		}
		else
		{
			// The tool tip can be used in this case.
			ItemName = CurrentToolTipText;
		}
		ItemDescriptor = InItemDescriptor;
		LastPickedWidget = Button;
		bIsUIWidget = true;
		return true;
	}
	return false;
}

#undef LOCTEXT_NAMESPACE
