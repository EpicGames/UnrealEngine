// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once


#include "Layout/WidgetPath.h"
#include "Serialization/JsonSerializable.h"
#include "Serialization/JsonSerializerMacros.h"

//
// UE::AIAssistant::SlateQuerier
//


namespace UE::AIAssistant::SlateQuerier
{
	/**
	 * Get the widget path under the cursor
	 */
	FWidgetPath GetWidgetPathUnderCursor();

	/**
	 * Initiates an AI Assistant query to describe a Slate widget.
	 */
	void QueryAIAssistantAboutSlateWidget(const FWidgetPath& InWidgetPath);

	/**
	 * Store structured context for query to pass along to AI Assistant as JSON.
	 */
	struct FSlateQueryStructuredContext : public FJsonSerializable
	{
		FString TopWindowName;
		FString UnrealEditorVersion;
		FString UnrealEditorLanguage;
		FString UnrealEditorLocale;
		TOptional<FString> WindowName;
		TOptional<FString> EditorName;
		TOptional<FString> EditorMode;
		TOptional<FString> ActiveToolName;
		TOptional<FString> TabName;
		TOptional<FString> ToolTipText;
		TOptional<FString> TextUnderCursor;
		TOptional<FString> WidgetType;
		TOptional<FString> ActorClass;
		TOptional<FString> ItemName;
		TOptional<FString> ItemDescriptor;
		TOptional<FString> CategoryName;
		TOptional<TArray<FString>> WidgetPath;

		BEGIN_JSON_SERIALIZER
		JSON_SERIALIZE("toplevel_window_name", TopWindowName);
		JSON_SERIALIZE("unreal_editor_version", UnrealEditorVersion);
		JSON_SERIALIZE("unreal_editor_language", UnrealEditorLanguage);
		JSON_SERIALIZE("unreal_editor_locale", UnrealEditorLocale);
		JSON_SERIALIZE_OPTIONAL("window_name", WindowName);
		JSON_SERIALIZE_OPTIONAL("editor_name", EditorName);
		JSON_SERIALIZE_OPTIONAL("editor_mode", EditorMode);
		JSON_SERIALIZE_OPTIONAL("active_tool", ActiveToolName);
		JSON_SERIALIZE_OPTIONAL("tab_name", TabName);
		JSON_SERIALIZE_OPTIONAL("tooltip_text", ToolTipText);
		JSON_SERIALIZE_OPTIONAL("text_under_cursor", TextUnderCursor);
		JSON_SERIALIZE_OPTIONAL("widget_type", WidgetType);
		JSON_SERIALIZE_OPTIONAL("actor_class", ActorClass);
		JSON_SERIALIZE_OPTIONAL("item_name", ItemName);
		JSON_SERIALIZE_OPTIONAL("item_descriptor", ItemDescriptor);
		JSON_SERIALIZE_OPTIONAL("category_name", CategoryName);
		JSON_SERIALIZE_OPTIONAL_ARRAY("widget_path", WidgetPath);
		END_JSON_SERIALIZER
	};

	/**
	 * Implements Slate widget querying functionality.
	 */
	class FSlateQuerier
	{
	public:
		FSlateQuerier(const FWidgetPath& InWidgetPath);

		/**
		 * Inspects widget path, determines query parameters, and sends query to Assistant backend.
		 */
		bool GenerateAndSendQuery();

		/**
		 * True if a query has been successfully constructed.
		 */
		bool IsValid() { return !GeneratedQuery.IsEmpty(); }

	protected:
		/**
		 * Inspect widget path to determine query parameters.
		 */
		void IdentifyObjectOfQuery();

		/**
		 * Generate query and hidden instructions to Assistant.
		 */
		void GenerateQuery();

		/**
		 * Generate context packet JSON to send to Assistant for additonal context clues.
		 */
		void GenerateStructuredContext();

		/**
		 * Send query off to Assistant backend API.
		 */
		void SendQuery();

		/**
		 * Determine editor or window info from the widget path for purposes of querying the Assistant.
		 * Return true if successful.
		 */
		bool FindEditorName();

		/**
		 * Determine tab or panel info from the widget path for purposes of querying the Assistant.
		 * Return true if successful.
		 */
		bool FindTabName();

		/**
		 * Return true and store information if widget path is in a level or asset editor.
		 */
		bool IsInEditor();

		/**
		 * Return true and store information if widget path is in a drawer overlay.
		 */
		bool IsInDrawer();

		/**
		 * Determine what type of widget the widget path is on for purposes of querying the Assistant.
		 * Return true if successful.
		 */
		bool FindItemName();

		/**
		 * Return true and store information if widget path is in a viewport.
		 */
		bool ItemIsInViewport();

		/**
		 * Return true and store information if widget path is in a graph node.
		 */
		bool ItemIsGraphNode();

		/**
		 * Return true and store information if widget path is in a search box.
		 */
		bool ItemIsSearchBox();

		/**
		 * Return true and store information if widget path is in a console input box.
		 */
		bool ItemIsConsoleInputBox();

		/**
		 * Return true and store information if widget path is in a Details panel property.
		 */
		bool ItemIsDetailsProperty();

		/**
		 * Return true and store information if widget path is in a plugin in Plugins browser.
		 */
		bool ItemIsPlugin();

		/**
		 * Return true and store information if widget path is on a navigation breadcrumb trail button.
		 */
		bool ItemIsBreadcrumbTrailButton();

		/**
		 * Return true and store information if widget path is on an asset in Content Browser.
		 */
		bool ItemIsAsset();

		/**
		 * Return true and store information if widget path is on an Actor in the Outliner panel, or on an
		 * instance or component in the SubobjectInstanceEditor.
		 */
		bool ItemIsOutlinerActorOrSubobject();

		/**
		 * Return true and store information if widget path is on a Details panel filter checkbox.
		 */
		bool ItemIsDetailPanelFilterCheckbox();

		/**
		 * Return true and store information if widget path is on a menu item.
		 */
		bool ItemIsMenuItem();

		/**
		 * Return true and store information if widget path is on a button.
		 */
		bool ItemIsButton();

		/**
		 * Helper method for ItemIsButton
		 * Return true and store information if widget path is on a button.
		 */
		bool ItemIsButtonType(const TSet<FName> InButtonWidgetTypes, FText InItemDescriptor);

		/**
		 * Helper method for buttons.  Returns true if method was able to successfully infer a name for the
		 * button.
		 */
		bool SetItemAsButton(const TSharedPtr<SWidget> Button, FText InItemDescriptor);


	private:
		FWidgetPath OriginalWidgetPath;
		FWidgetPath WidgetPath;
		FText CurrentToolTipText;
		FText FormattedWindowName;
		FText FormattedTabName;
		FText TextUnderCursor;
		FText ItemName;
		FText ItemDescriptor;
		FString SelectedActorName;
		FString SelectedActorClass;
		TSharedPtr<SWidget> LastPickedWidget;

		bool bIsUIWidget = false;
		bool bIsObject = false;
		bool bIsInAssistantPanel = false;

		FText GeneratedQuery;
		FText GeneratedQueryInstructions;
		FText LanguageInstructions;

		FSlateQueryStructuredContext StructuredContext;
	};
};
