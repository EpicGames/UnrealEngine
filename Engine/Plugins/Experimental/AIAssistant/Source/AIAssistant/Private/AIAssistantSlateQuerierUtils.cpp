// Copyright Epic Games, Inc. All Rights Reserved.

#include "AIAssistantSlateQuerierUtils.h"

#include "Framework/Application/SlateApplication.h"
#include "Internationalization/Regex.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Text/SRichTextBlock.h"


using namespace UE::AIAssistant::SlateQuerier;

FString Utility::CleanExtraWhiteSpaceFromString(const FString& InString)
{
	FString CleanString = "";
	static const FRegexPattern WhiteSpacePattern(TEXT("(\\S+)"));
	FRegexMatcher Matcher(WhiteSpacePattern, InString);
	while (Matcher.FindNext())
	{
		if (!CleanString.IsEmpty())
		{
			CleanString += " ";
		}
		CleanString += Matcher.GetCaptureGroup(1);
	}

	return MoveTemp(CleanString);
}

FText Utility::GetWidgetText(const TSharedRef<SWidget> InWidget)
{
	FText OutText;
	// refuse any that only contain numbers (they're probably just values)
	const FRegexPattern AlphaPattern(TEXT("[A-Za-z]+"));
	FText WidgetText;

	if (InWidget->GetType() == "STextBlock")
	{
		TSharedRef<STextBlock> TextBlock = StaticCastSharedRef<STextBlock>(InWidget);
		WidgetText = TextBlock->GetText();
	}
	else if (InWidget->GetType() == "SRichTextBlock")
	{
		TSharedRef<SRichTextBlock> TextBlock = StaticCastSharedRef<SRichTextBlock>(InWidget);
		WidgetText = TextBlock->GetText();
	}
	if (!WidgetText.IsEmpty())
	{
		FRegexMatcher AlphaMatcher(AlphaPattern, WidgetText.ToString());
		if (AlphaMatcher.FindNext())
		{
			OutText = WidgetText;
		}
	}

	return OutText;
}

FText Utility::FindTextUnderCursor(const FWidgetPath& InWidgetPath)
{
	return InWidgetPath.IsValid() ? Utility::GetWidgetText(InWidgetPath.GetLastWidget()) : FText();
}

FText Utility::FindChildWidgetWithText(const TSharedPtr<SWidget> InWidget)
{
	if (InWidget.IsValid())
	{
		FText WidgetText = Utility::GetWidgetText(InWidget.ToSharedRef());
		if (!WidgetText.IsEmpty())
		{
			return WidgetText;
		}

		// get all children and see if any are text widgets.
		for (int32 ChildIdx = 0; ChildIdx < InWidget->GetChildren()->Num(); ChildIdx++)
		{
			TSharedRef<SWidget> ThisWidget = InWidget->GetChildren()->GetChildAt(ChildIdx);
			const FText ChildText = Utility::FindChildWidgetWithText(ThisWidget.ToSharedPtr());
			if (!ChildText.IsEmpty())
			{
				return ChildText;
			}
		}
	}

	return FText();
}

FText Utility::FindCurrentToolTipText()
{
	for (const TArray<TSharedRef<SWindow>> AllWindows = FSlateApplication::Get().GetTopLevelWindows();
		const TSharedRef<SWindow>& ThisWindow : AllWindows)
	{
		if (ThisWindow->GetType() == EWindowType::ToolTip && ThisWindow->IsVisible())
		{
			return Utility::FindChildWidgetWithText(ThisWindow);
		}
	}

	return FText();
}
