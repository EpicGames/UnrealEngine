// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Layout/WidgetPath.h"
#include "Templates/SharedPointer.h"


namespace UE::AIAssistant::SlateQuerier::Utility
{
	/**
	 * Return copy of InString cleaned of redundant whitespace.
	 */
	FString CleanExtraWhiteSpaceFromString(const FString& InString);

	/**
	 * Look for text in widget under cursor and return it.  Return empty FText if not found.
	 */
	FText FindTextUnderCursor(const FWidgetPath& InWidgetPath);

	/**
	 * Return text from InWidget if it is a text item.  Return empty FText if not found.
	 */
	FText GetWidgetText(const TSharedRef<SWidget> InWidget);

	/**
	 * Search children of InWidget and return text from first one found with alphabetic text.
	 * Return empty FText if not found.
	 */
	FText FindChildWidgetWithText(const TSharedPtr<SWidget> InWidget);

	/**
	 * Return first active tooltip text found.  Return empty FText if not found.
	 */
	FText FindCurrentToolTipText();
};
