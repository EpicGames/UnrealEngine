// Copyright Epic Games, Inc. All Rights Reserved.

#include "AIAssistantSlateQuerierUtilsTypeSearch.h"

using namespace UE::AIAssistant::SlateQuerier;

TSharedPtr<SWidget> Utility::FindClosestWidgetPtrOfTypes(const FWidgetPath& InWidgetPath, const TSet<FName> InWidgetTypes)
{
	TSharedPtr<SWidget> ClosestWidget;
	for (int32 WidgetIndex = InWidgetPath.Widgets.Num() - 1; WidgetIndex >= 0; --WidgetIndex)
	{
		const FArrangedWidget* ArrangedWidget = &InWidgetPath.Widgets[WidgetIndex];
		const TSharedRef<SWidget>& ThisWidget = ArrangedWidget->Widget;
		if (InWidgetTypes.Contains(ThisWidget->GetType()))
		{
			ClosestWidget = ThisWidget.ToSharedPtr();
			break;
		}
	}
	return ClosestWidget;
}

TSharedPtr<SWidget> Utility::FindClosestWidgetPtrOfType(const FWidgetPath& InWidgetPath, const FName& InWidgetType)
{
	const TSet<FName> WidgetTypes = TSet<FName>({InWidgetType});
	return Utility::FindClosestWidgetPtrOfTypes(InWidgetPath, WidgetTypes);
}

TSharedPtr<SWidget> Utility::FindClosestWidgetPtrOfSortedTypes(
	const FWidgetPath& InWidgetPath, const TConstArrayView<FName>& InWidgetTypes)
{
	for (FName WidgetType : InWidgetTypes)
	{
		TSharedPtr<SWidget> ClosestWidget = Utility::FindClosestWidgetPtrOfType(InWidgetPath, WidgetType);
		if (ClosestWidget.IsValid())
		{
			return ClosestWidget;
		}
	}

	return nullptr;
}

TSharedPtr<SWidget> Utility::FindFirstWidgetPtrOfType(const FWidgetPath& InWidgetPath, const FName& InWidgetType)
{
	TSharedPtr<SWidget> FirstWidget;
	for (int32 WidgetIndex = 0; WidgetIndex < InWidgetPath.Widgets.Num(); WidgetIndex++)
	{
		const FArrangedWidget* ArrangedWidget = &InWidgetPath.Widgets[WidgetIndex];
		const TSharedRef<SWidget>& ThisWidget = ArrangedWidget->Widget;
		if (ThisWidget->GetType() == InWidgetType)
		{
			FirstWidget = ThisWidget.ToSharedPtr();
			break;
		}
	}
	return FirstWidget;
}

void Utility::FindChildWidgetsOfType(TArray<TSharedRef<SWidget>>& OutWidgets, TSharedPtr<SWidget> InWidget, const FName& InWidgetType)
{
	if (InWidget.IsValid())
	{
		for (int32 ChildIdx = 0; ChildIdx < InWidget->GetChildren()->Num(); ChildIdx++)
		{
			TSharedRef<SWidget> ThisWidget = InWidget->GetChildren()->GetChildAt(ChildIdx);
			if (ThisWidget->GetType() == InWidgetType)
			{
				OutWidgets.Add(ThisWidget);
			}
			Utility::FindChildWidgetsOfType(OutWidgets, ThisWidget.ToSharedPtr(), InWidgetType);
		}
	}
}



TSharedPtr<SWidget> Utility::FindChildWidgetOfType(const TSharedPtr<SWidget> InWidget, const FName& InWidgetType)
{
	if (InWidget.IsValid())
	{
		for (int32 ChildIdx = 0; ChildIdx < InWidget->GetChildren()->Num(); ChildIdx++)
		{
			TSharedRef<SWidget> ThisWidget = InWidget->GetChildren()->GetChildAt(ChildIdx);
			if (ThisWidget->GetType() == InWidgetType)
			{
				return ThisWidget.ToSharedPtr();
			}
			else
			{
				TSharedPtr<SWidget> ChildWidget = Utility::FindChildWidgetOfType(ThisWidget.ToSharedPtr(), InWidgetType);
				if (ChildWidget.IsValid() && ChildWidget->GetType() == InWidgetType)
				{
					return ChildWidget;
				}
			}
		}
	}
	return TSharedPtr<SWidget>();
}

TSharedPtr<SWidget> Utility::FindClosestMenuItem(const FWidgetPath& WidgetPath)
{
	TSharedPtr<SWidget> MenuItemBlock;
	if (WidgetPath.IsValid())
	{
		// is it a menu item? Look up for an SMenuEntryBlock or SWidgetBlock.
		static const TStaticArray<FName, 2> MenuTypes = {"SMenuEntryBlock", "SWidgetBlock"};
		MenuItemBlock = Utility::FindClosestWidgetPtrOfSortedTypes(WidgetPath, MenuTypes);

		// disabled menu items *contain* the menu entry block
		if (!MenuItemBlock.IsValid())
		{
			if (WidgetPath.GetLastWidget()->GetChildren()->Num() > 0)
			{
				for (int32 ChildIdx = 0; ChildIdx < WidgetPath.GetLastWidget()->GetChildren()->Num(); ChildIdx++)
				{
					TSharedRef<SWidget> ThisWidget = WidgetPath.GetLastWidget()->GetChildren()->GetChildAt(ChildIdx);
					for (FName MenuType : MenuTypes)
					{
						if (ThisWidget->GetType() == MenuType)
						{
							MenuItemBlock = ThisWidget.ToSharedPtr();
							break;
						}
					}
				}
			}
		}
	}

	return MenuItemBlock;
}
