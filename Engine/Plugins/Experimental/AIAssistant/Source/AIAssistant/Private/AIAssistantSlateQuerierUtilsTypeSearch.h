// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Layout/WidgetPath.h"


namespace UE::AIAssistant::SlateQuerier::Utility
{
	/**
	 * Search backwards from end of InWidgetPath and return first widget that matches any type in WidgetTypes.
	 * Return null ptr if not found.
	 */
	TSharedPtr<SWidget> FindClosestWidgetPtrOfTypes(const FWidgetPath& InWidgetPath, const TSet<FName> InWidgetTypes);

	/**
	 * Search backwards from end of InWidgetPath and return first widget that matches WidgetType.
	 * Return null ptr if not found.
	 */
	TSharedPtr<SWidget> FindClosestWidgetPtrOfType(const FWidgetPath& InWidgetPath, const FName& InWidgetType);

	/**
	 * Search backwards from end of InWidgetPath and return first widget that matches a type in WidgetTypes,
	 * testing each type in order.
	 * Return null ptr if not found.
	 */
	TSharedPtr<SWidget> FindClosestWidgetPtrOfSortedTypes(
		const FWidgetPath& InWidgetPath, const TConstArrayView<FName>& InWidgetTypes);

	/**
	 * Search forward from start of InWidgetPath and return first widget that matches WidgetType.
	 * Return null ptr if not found.
	 */
	TSharedPtr<SWidget> FindFirstWidgetPtrOfType(const FWidgetPath& InWidgetPath, const FName& InWidgetType);

	/**
	 * Search children of InWidget for any widgets that match InWidgetType; return all found widgets in OutWidgets.
	 */
	void FindChildWidgetsOfType(TArray<TSharedRef<SWidget>>& OutWidgets, TSharedPtr<SWidget> InWidget, const FName& InWidgetType);

	/**
	 * Search children of InWidget and return first widget that matches InWidgetType.
	 * Return null ptr if not found.
	 */
	TSharedPtr<SWidget> FindChildWidgetOfType(const TSharedPtr<SWidget> InWidget, const FName& InWidgetType);

	/**
	 * Search backwards from end of InWidgetPath and return first menu item type widget found.
	 * Return null ptr if not found.
	 */
	TSharedPtr<SWidget> FindClosestMenuItem(const FWidgetPath& InWidgetPath);
};
