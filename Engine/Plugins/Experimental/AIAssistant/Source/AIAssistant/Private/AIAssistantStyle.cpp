// Copyright Epic Games, Inc. All Rights Reserved.


#include "AIAssistantStyle.h"

#include "Styling/SlateStyleRegistry.h"
#include "Framework/Application/SlateApplication.h"
#include "Slate/SlateGameResources.h"
#include "Interfaces/IPluginManager.h"
#include "Styling/SlateStyleMacros.h"


#define RootToContentDir Style->RootToContentDir


TSharedPtr<FSlateStyleSet> FAIAssistantStyle::StyleInstance = nullptr;


static const FVector2D Icon16x16(16.0f, 16.0f);
static const FVector2D Icon18x18(18.0f, 18.0f);
static const FVector2D Icon20x20(20.0f, 20.0f);


void FAIAssistantStyle::Initialize()
{
	if (!StyleInstance.IsValid())
	{
		StyleInstance = Create();
		FSlateStyleRegistry::RegisterSlateStyle(*StyleInstance);
	}
}


void FAIAssistantStyle::Shutdown()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*StyleInstance);
	ensure(StyleInstance.IsUnique());
	StyleInstance.Reset();
}


FName FAIAssistantStyle::GetStyleSetName()
{
	static FName StyleSetName(TEXT("AIAssistantStyle"));
	return StyleSetName;
}


TSharedRef<FSlateStyleSet> FAIAssistantStyle::Create()
{
	TSharedRef<FSlateStyleSet> Style = MakeShareable(new FSlateStyleSet("AIAssistantStyle"));
	Style->SetContentRoot(IPluginManager::Get().FindPlugin(UE_PLUGIN_NAME)->GetBaseDir() / TEXT("Resources") / TEXT("Icons"));
		
	Style->Set("AIAssistant.OpenPluginWindow", new IMAGE_BRUSH_SVG(TEXT("AIAssistant"), Icon16x16));
	Style->Set("AIAssistant.Copy", new IMAGE_BRUSH_SVG(TEXT("Copy"), Icon18x18));
	
	return Style;
}


void FAIAssistantStyle::ReloadTextures()
{
	if (FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().GetRenderer()->ReloadTextureResources();
	}
}


const ISlateStyle& FAIAssistantStyle::Get()
{
	return *StyleInstance;
}
