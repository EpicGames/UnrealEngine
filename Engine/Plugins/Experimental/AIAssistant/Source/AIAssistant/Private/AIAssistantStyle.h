// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once


#include "Styling/SlateStyle.h"


//
// FAIAssistantStyle
//


class FAIAssistantStyle
{
public:

	
	static void Initialize();
	static void Shutdown();

	static void ReloadTextures();

	static const ISlateStyle& Get();

	static FName GetStyleSetName();

	
private:

	
	static TSharedRef< class FSlateStyleSet > Create();

	static TSharedPtr< class FSlateStyleSet > StyleInstance;
};