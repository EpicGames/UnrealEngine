// Copyright Epic Games, Inc. All Rights Reserved.

#include "AIAssistantSubsystem.h"

#include "Containers/UnrealString.h"
#include "Editor.h"
#include "Modules/ModuleManager.h"
#include "Templates/SharedPointer.h"
#include "Templates/ValueOrError.h"
#include "UObject/ObjectPtr.h"

#include "AIAssistant.h"
#include "AIAssistantLog.h"
#include "AIAssistantToolset.h"
#include "AIAssistantWebBrowser.h"
#include "ToolsetRegistry/UToolsetRegistry.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AIAssistantSubsystem)


using namespace UE::AIAssistant;

//
// Statics.
//

TValueOrError<TObjectPtr<UAIAssistantSubsystem>, FString> UAIAssistantSubsystem::Get(
	const FString& WarningMessage)
{
	// Local function to construct error to return and optionally log a warning.
	auto MaybeWarnOnError = [&WarningMessage](const FString& ErrorMessage)
	{
		if (!WarningMessage.IsEmpty())
		{
			UE_LOGF(LogAIAssistant, Warning, "%ls: %ls", *WarningMessage, *ErrorMessage);
		}
		return MakeError(ErrorMessage);
	};

	if (!GEditor)
	{
		return MaybeWarnOnError(TEXT("Editor is not available"));
	}

	TObjectPtr<UAIAssistantSubsystem> AssistantSubsystem(
		GEditor->GetEditorSubsystem<UAIAssistantSubsystem>());

	if (!AssistantSubsystem)
	{
		return MaybeWarnOnError(TEXT("AI Assistant subsystem is not available"));
	}

	return MakeValue(AssistantSubsystem);
}

FAIAssistantModule& UAIAssistantSubsystem::GetAIAssistantModule()
{
	return FModuleManager::LoadModuleChecked<FAIAssistantModule>(UE_PLUGIN_NAME);
}

// Get the assistant web browser.
TSharedPtr<SAIAssistantWebBrowser> UAIAssistantSubsystem::GetAIAssistantWebBrowserWidget()
{
	auto AIAssistantWebBrowserWidget = GetAIAssistantModule().GetAIAssistantWebBrowserWidget();
	check(AIAssistantWebBrowserWidget.IsValid());
	return AIAssistantWebBrowserWidget;
}

//
// UAIAssistantSubsystem
//

void UAIAssistantSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);
	UToolsetRegistry::RegisterToolsetClass(UAIAssistantToolset::StaticClass());
}

void UAIAssistantSubsystem::Deinitialize()
{
	UToolsetRegistry::UnregisterToolsetClass(UAIAssistantToolset::StaticClass());
	Super::Deinitialize();
}

/*no:static*/ void UAIAssistantSubsystem::ShowContextMenuViaJavaScript(const FString& SelectedString, const int32 ClientX, const int32 ClientY) const
{
	GetAIAssistantModule().ShowContextMenu(SelectedString, FVector2f(ClientX, ClientY));
}

