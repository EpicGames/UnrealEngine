// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Containers/UnrealString.h"
#include "EditorSubsystem.h"
#include "Templates/SharedPointer.h"
#include "Templates/ValueOrError.h"
#include "UObject/ObjectPtr.h"

#include "AIAssistantSubsystem.generated.h"


class FAIAssistantModule;
class SAIAssistantWebBrowser;

//
// UAIAssistantSubsystem
//

UCLASS(BlueprintType)
class UAIAssistantSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()

public:
	// UEditorSubsystem interface
	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	virtual void Deinitialize() override;

	//
	// JavaScript exposed functions.
	//
	
	// See NOTE_JAVASCRIPT_CPP_FUNCTIONS in C++ code for how to call this from JavaScript.
	UFUNCTION(BlueprintCallable, Category="JavaScript")
	void ShowContextMenuViaJavaScript(const FString& SelectedString, const int32 ClientX, const int32 ClientY) const;

public:
	// Get the assistant subsystem.  Optionally log a warning if passed a non-empty string (the warning
	// message will start with the passed-in string).
	static TValueOrError<TObjectPtr<UAIAssistantSubsystem>, FString> Get(const FString& WarningMessage=FString());

	// Get the assistant module.
	static FAIAssistantModule& GetAIAssistantModule();
	// Get the assistant web browser.
	static TSharedPtr<SAIAssistantWebBrowser> GetAIAssistantWebBrowserWidget();
};
