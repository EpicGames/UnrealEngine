// Copyright Epic Games, Inc. All Rights Reserved.

#include "AIAssistantTextMessage.h"

#include "Containers/UnrealString.h"

#include "AIAssistantWebApi.h"

namespace UE::AIAssistant
{
	FAddMessageToConversationOptions CreateUserMessage(
		const FString& VisiblePrompt, const FString& HiddenContext)
	{
		FAddMessageToConversationOptions Options;
		auto& Message = Options.Message;
		Message.MessageRole = EMessageRole::User;
		auto& MessageContent = Message.MessageContent;
		for (const auto& PromptAndVisible : {
				TPair<const FString&, bool>(VisiblePrompt, true),
				TPair<const FString&, bool>(HiddenContext, false),
			})
		{
			const auto& Prompt = PromptAndVisible.Key;
			bool bVisible = PromptAndVisible.Value;
			if (Prompt.IsEmpty()) continue;

			auto& MessageContentItem = MessageContent.Emplace_GetRef();
			MessageContentItem.bVisibleToUser = bVisible;
			MessageContentItem.ContentType = EMessageContentType::Text;
			MessageContentItem.Content.Emplace<FTextMessageContent>();
			MessageContentItem.Content.Get<FTextMessageContent>().Text = Prompt;
		}
		return MoveTemp(Options);
	}
}