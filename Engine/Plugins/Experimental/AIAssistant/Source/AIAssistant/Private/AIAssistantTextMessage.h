// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"

#include "AIAssistantWebApi.h"

namespace UE::AIAssistant
{
	// Create a user message from visible and hidden prompt.
	FAddMessageToConversationOptions CreateUserMessage(
		const FString& VisiblePrompt, const FString& HiddenContext);
	
}