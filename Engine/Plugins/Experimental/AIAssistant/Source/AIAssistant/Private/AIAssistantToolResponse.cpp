// Copyright Epic Games, Inc. All Rights Reserved.

#include "AIAssistantToolResponse.h"

#include "Async/UniqueLock.h"
#include "Internationalization/Culture.h"
#include "Editor.h"
#include "Misc/EngineVersion.h"

#include "AIAssistantRunSequential.h"
#include "AIAssistantSubsystem.h"

namespace UE::AIAssistant
{
	FAddMessageToConversationOptions CreateToolResponseMessage(
		const FString& ToolName,
		const FString& ToolCallId,
		const TValueOrError<FString, FString>& ToolResult)
	{
		// Create a user message to add tool responses to.
		FAddMessageToConversationOptions Options;
		auto& ResponseMessage = Options.Message;
		ResponseMessage.MessageRole = EMessageRole::User;
		auto& MessageContent = Options.Message.MessageContent.Emplace_GetRef();
		MessageContent.bVisibleToUser = false;
		MessageContent.ContentType = EMessageContentType::ToolResponse;
		MessageContent.Content.Emplace<FToolResponseContent>();
		auto& ToolResponse = MessageContent.Content.Get<FToolResponseContent>();
		ToolResponse.Name = ToolName;
		ToolResponse.ToolCallId = ToolCallId;
		ToolResponse.Success.Emplace(ToolResult.HasValue());
		if (*ToolResponse.Success)
		{
			ToolResponse.ResponseRawJson = ToolResult.GetValue();
		}
		else
		{
			// The Backend currently expects a response to be present, even
			// when an error occurs. This should be removed when that is fixed.
			ToolResponse.ResponseRawJson = TEXT("{}");
			ToolResponse.ErrorMessage = ToolResult.GetError();
		}
		return Options;
	}
}