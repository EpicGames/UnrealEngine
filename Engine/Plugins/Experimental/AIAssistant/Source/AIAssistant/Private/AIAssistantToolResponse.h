// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Templates/ValueOrError.h"

#include "AIAssistantWebApi.h"

namespace UE::AIAssistant
{
	// Create a tool response message content.
	FAddMessageToConversationOptions CreateToolResponseMessage(
		const FString& ToolName, const FString& ToolCallId,
		const TValueOrError<FString, FString>& ToolResult);
	
}