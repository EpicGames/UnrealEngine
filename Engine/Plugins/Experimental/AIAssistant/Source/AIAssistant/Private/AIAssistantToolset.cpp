// Copyright Epic Games, Inc. All Rights Reserved.

#include "AIAssistantToolset.h"

#include "BlueprintEditor.h"
#include "EdGraphUtilities.h"
#include "IMaterialEditor.h"
#include "IPythonScriptPlugin.h"
#include "MaterialGraph/MaterialGraph.h"
#include "Materials/Material.h"
#include "UObject/ObjectPtr.h"
#include "Widgets/Docking/SDockTab.h"

#include "AIAssistant.h"
#include "AIAssistantSubsystem.h"
#include "AIAssistantSlateQuerierUtils.h"
#include "AIAssistantSlateQuerierUtilsTypeSearch.h"
#include "AIAssistantWebBrowser.h"

using namespace UE::AIAssistant::SlateQuerier;

#include UE_INLINE_GENERATED_CPP_BY_NAME(AIAssistantToolset)


FAIAssistantContext UAIAssistantToolset::GetProjectContext()
{
	FAIAssistantContext CombinedContext;

	TArray<FString> EngineTips({
		TEXT("Folder paths always need to end in /"),
		TEXT("100 units is 1 meter"),
		TEXT("Screen space coordinates are normalized coordinates where 0, 0 is the upper left corner "
			 "and 1, 1 is the lower right corner."),
		TEXT("Colors with a max of 1 are LDR. No max means HDR."),
		TEXT("If you aren't sure what the user is referring to, try to see what they have selected."),
		TEXT("Class refPath is the full soft class path e.g. /Script/Engine.Actor."),
	});

	CombinedContext.UnrealContext = FString::Join(EngineTips, TEXT("\n"));

	const UAIAssistantContextProject* ProjectSettings = GetDefault<UAIAssistantContextProject>();
	if (ProjectSettings)
	{
		CombinedContext.ProjectContext = ProjectSettings->Prompt;		
	}
	const UAIAssistantContextUser* UserSettings = GetDefault<UAIAssistantContextUser>();
	if (UserSettings)
	{
		CombinedContext.UserContext = UserSettings->Prompt;		
	}
	return CombinedContext;
}

FAIAssistantDockContext UAIAssistantToolset::GetDockedContext()
{
	TSharedPtr<SAIAssistantWebBrowser> BrowserWidget =
		UAIAssistantSubsystem::GetAIAssistantModule().GetAIAssistantWebBrowserWidget();
	if (!BrowserWidget)
	{
		return FAIAssistantDockContext();
	}

	// Walk up the widget hierarchy until we encounter an asset editor.
	TSharedPtr<SWidget> Current = BrowserWidget;
	TSharedPtr<SWidget> StandaloneAssetEditorHost;
	while (Current)
	{
		if (Current->GetTypeAsString() == "SStandaloneAssetEditorToolkitHost")
		{
			StandaloneAssetEditorHost = Current;
			break;
		}
		Current = Current->GetParentWidget();
	}

	// If we found an asset editor, we'll try drill down through the tab system to get the asset editor.
	if (StandaloneAssetEditorHost)
	{
		TArray<TSharedRef<SWidget>> TabStacks;
		Utility::FindChildWidgetsOfType(TabStacks, StandaloneAssetEditorHost, "SDockingTabStack");
		for (auto& ThisTabStack : TabStacks)
		{
			TArray<TSharedRef<SWidget>> DockTabs;
			Utility::FindChildWidgetsOfType(DockTabs, ThisTabStack.ToSharedPtr(), "SDockTab");

			for (auto& ThisTab : DockTabs)
			{
				TSharedRef<SDockTab> ThisTabCast = StaticCastSharedRef<SDockTab>(ThisTab);
				if (ThisTabCast->IsForeground())
				{
					TSharedPtr<FTabManager> PickedWidgetTabManager = ThisTabCast->GetTabManagerPtr();

					if (PickedWidgetTabManager.IsValid())
					{
						UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
						TArray<UObject*> AllEditedAssets = AssetEditorSubsystem->GetAllEditedAssets();
						for (auto& ThisEditedAsset : AllEditedAssets)
						{
							IAssetEditorInstance* ThisEditor = AssetEditorSubsystem->FindEditorForAsset(ThisEditedAsset, false);
							if (ThisEditor && ThisEditor->GetAssociatedTabManager() == PickedWidgetTabManager)
							{
								// This per-asset logic is not ideal, but it's the best we can do with the current AssetEditorInterface.
								// If/when the asset editor interface is improved, we can simplify this logic.
								FAIAssistantDockContext AssetInfo;
								AssetInfo.Asset = ThisEditedAsset;
								FGraphPanelSelectionSet SelectedNodes;
								if (ThisEditedAsset->IsA<UBlueprint>())
								{
									FBlueprintEditor* BlueprintEditor = static_cast<FBlueprintEditor*>(ThisEditor);
									AssetInfo.Graph = BlueprintEditor->GetFocusedGraph();
									SelectedNodes = BlueprintEditor->GetSelectedNodes();
								}
								else if (UMaterial* Material = Cast<UMaterial>(ThisEditedAsset))
								{
									IMaterialEditor* MaterialEditor = static_cast<IMaterialEditor*>(ThisEditor);
									AssetInfo.Graph = Material->MaterialGraph.Get();
									SelectedNodes = MaterialEditor->GetSelectedNodes();
								}
								for (auto& Node : SelectedNodes)
								{
									if (UEdGraphNode* GraphNode = Cast<UEdGraphNode>(Node))
									{
										AssetInfo.SelectedNodes.Add(GraphNode);
									}
								}
								return AssetInfo;
							}
						}
					}
				}
			}
		}
	}

	return FAIAssistantDockContext();
}
