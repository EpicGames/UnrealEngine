// Copyright Epic Games, Inc. All Rights Reserved. 

#pragma once

#include "Engine/DeveloperSettings.h"

#include "ToolsetRegistry/ToolsetDefinition.h"

#include "AIAssistantToolset.generated.h"

class UEdGraph;
class UEdGraphNode;

UCLASS(config=EditorPerProjectUserSettings, meta=(DisplayName="Epic Developer Assistant"))
class UAIAssistantContextUser : public UDeveloperSettings
{
	GENERATED_BODY()
public:
	/// This field is used to help the Epic Developer Assistant understand important info about you,
	/// and can include things like your role, preferences, current goals, etc. The EDA
	/// will automatically consider this information whenever you ask it to do anything.
	UPROPERTY(Config, EditAnywhere, Category="Epic Developer Assistant", Meta=(Multiline=true))
	FString Prompt;
};

UCLASS(config=Editor, defaultconfig, meta=(DisplayName="Epic Developer Assistant"))
class UAIAssistantContextProject : public UDeveloperSettings
{
	GENERATED_BODY()
public:
	/// This field is used to help the Epic Developer Assistant understand important info about your
	/// project, and can include things like game genre, naming conventions, art style, etc. The EDA
	/// will automatically consider this information whenever you ask it to do anything.
	UPROPERTY(Config, EditAnywhere, Category = "Epic Developer Assistant", Meta=(Multiline=true))
	FString Prompt;
};

/// Contains information about the current Unreal project and user.
USTRUCT(BlueprintType)
struct FAIAssistantContext
{
	GENERATED_USTRUCT_BODY()
public:
	/// Describes important context about how to work with the Unreal engine.
	UPROPERTY(BlueprintReadOnly, Category = "AIAssistant")
	FString UnrealContext;

	/// Describes important context about the currently active Unreal project.
	UPROPERTY(BlueprintReadOnly, Category = "AIAssistant")
	FString ProjectContext;

	/// Describes important context about the currently active UnrealEd user.
	UPROPERTY(BlueprintReadOnly, Category = "AIAssistant")
	FString UserContext;
};

/// Contains information about the state of the editor that the assistant is currently docked with.
USTRUCT(BlueprintType)
struct FAIAssistantDockContext
{
	GENERATED_USTRUCT_BODY()
public:
	/// The asset instance that is currently being edited (if any).
	UPROPERTY(BlueprintReadOnly, Category = "AIAssistant")
	TObjectPtr<UObject> Asset;

	/// The graph instance that is currently being edited (if any).
	UPROPERTY(BlueprintReadOnly, Category = "AIAssistant")
	TObjectPtr<UEdGraph> Graph;	

	/// The list of selected graph nodes edited (if any).
	UPROPERTY(BlueprintReadOnly, Category = "AIAssistant")
	TArray<TObjectPtr<UEdGraphNode>> SelectedNodes;
};

/// These functions help the AI assistant inspect its state as it relates to the larger Unreal Editor.
UCLASS(BlueprintType, Hidden)
class UAIAssistantToolset : public UToolsetDefinition
{
	GENERATED_BODY()

public:
	/// Returns important information about the current project and/or current user.
	UFUNCTION(meta = (AICallable), Category = "AI Assistant")
	static FAIAssistantContext GetProjectContext();

	/// Returns information about the part of the unreal UI that the assistant is docked into.
	/// For example, if the assistant is docked with a Blueprint asset editor, this returns
	/// the Blueprint asset that is being edited, which graph is in focus, etc.  If the assistant
	/// is closed or not docked to an asset editor, this information will be empty.
	UFUNCTION(meta = (AICallable), Category = "AI Assistant")
	static FAIAssistantDockContext GetDockedContext();
};
