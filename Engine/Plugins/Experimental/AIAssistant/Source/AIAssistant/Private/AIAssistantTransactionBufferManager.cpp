// Copyright Epic Games, Inc. All Rights Reserved.

#include "AIAssistantTransactionBufferManager.h"

#include "CoreGlobals.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Editor/TransBuffer.h"
#include "HAL/IConsoleManager.h"
#include "Misc/ConfigCacheIni.h"

#include "AIAssistantLog.h"

namespace
{
	// Encapsulates all mutable state for the transaction buffer manager.
	struct FTransactionBufferManagerState
	{
		// The original global transaction buffer.
		TObjectPtr<UTransactor> SavedGlobalBuffer;

		// Whether an override is currently active.
		bool bIsOverrideActive = false;

		// Registry of plugin-created transaction buffers.
		TMap<FString, TObjectPtr<UTransBuffer>> BufferRegistry;

		// Reset all state to initial values.
		void Reset()
		{
			SavedGlobalBuffer = nullptr;
			bIsOverrideActive = false;
			BufferRegistry.Empty();
		}
	};

	static FTransactionBufferManagerState TransactionBufferManagerState;
}  // namespace

namespace UE::AIAssistant
{

static constexpr int32 DefaultUndoBufferSizeMb = 256;

TObjectPtr<UTransBuffer> FTransactionBufferManager::GetTransactionBuffer(const FString& BufferName)
{
	check(IsInGameThread());

	if (TObjectPtr<UTransBuffer>* Result =
		TransactionBufferManagerState.BufferRegistry.Find(BufferName))
	{
		return *Result;
	}
	return nullptr;
}

TObjectPtr<UTransBuffer> FTransactionBufferManager::GetOrCreateTransactionBuffer(
	const FString& BufferName)
{
	check(IsInGameThread());
	check(GConfig);

	if (TObjectPtr<UTransBuffer> ExistingBuffer = GetTransactionBuffer(BufferName))
	{
		return ExistingBuffer;
	}

	TObjectPtr<UTransBuffer> NewBuffer = NewObject<UTransBuffer>();
	check(NewBuffer);

	// This replicates the behavior of the global undo buffer initialization found in
	// UEditorEngine::CreateTrans().
	int32 UndoBufferSize = -1;
	if (!GConfig->GetInt(TEXT("Undo"), TEXT("UndoBufferSize"),
		UndoBufferSize, GEditorPerProjectIni) || UndoBufferSize < 0)
	{
		UndoBufferSize = DefaultUndoBufferSizeMb;
	}
	NewBuffer->AddToRoot();
	NewBuffer->Initialize(static_cast<SIZE_T>(UndoBufferSize) * 1024 * 1024);
	TransactionBufferManagerState.BufferRegistry.Add(BufferName, NewBuffer);
	return NewBuffer;
}

bool FTransactionBufferManager::DestroyTransactionBuffer(const FString& BufferName)
{
	check(IsInGameThread());
	check(GEditor);

	TObjectPtr<UTransBuffer>* BufferPtr =
		TransactionBufferManagerState.BufferRegistry.Find(BufferName);
	if (!BufferPtr)
	{
		return false;
	}

	UTransBuffer* Buffer = BufferPtr->Get();

	// Cannot destroy a buffer that is currently the active override
	if (GEditor->Trans == Buffer)
	{
		UE_LOGF(
			LogAIAssistant, Warning,
			"Cannot destroy transaction buffer '%ls' while it is active as override",
			*BufferName);
		return false;
	}

	// Cannot destroy the saved global buffer while an override is active
	if (TransactionBufferManagerState.SavedGlobalBuffer == Buffer)
	{
		UE_LOGF(
			LogAIAssistant, Warning,
			"Cannot destroy transaction buffer '%ls' while it is saved as the global buffer",
			*BufferName);
		return false;
	}

	// Remove from root to allow garbage collection
	if (Buffer)
	{
		Buffer->RemoveFromRoot();
	}

	TransactionBufferManagerState.BufferRegistry.Remove(BufferName);
	return true;
}

void FTransactionBufferManager::Shutdown()
{
	check(IsInGameThread());

	// Restore override back to original global buffer if active
	if (TransactionBufferManagerState.bIsOverrideActive)
	{
		RestoreGlobalBuffer();
	}

	// Remove all buffers from root and clear registry
	for (auto& Pair : TransactionBufferManagerState.BufferRegistry)
	{
		if (Pair.Value)
		{
			Pair.Value->RemoveFromRoot();
		}
	}
	TransactionBufferManagerState.Reset();
}

bool FTransactionBufferManager::SetOverrideBuffer(UTransactor* InOverrideBuffer)
{
	check(IsInGameThread());
	check(GEditor);

	if (!InOverrideBuffer)
	{
		UE_LOGF(
			LogAIAssistant, Warning,
			"SetOverrideBuffer: OverrideBuffer is null, use RestoreGlobalBuffer() to restore");
		return false;
	}

	if (TransactionBufferManagerState.bIsOverrideActive)
	{
		UE_LOGF(
			LogAIAssistant, Warning,
			"SetOverrideBuffer: Cannot set override while another override is active. "
				 "Call RestoreGlobalBuffer() first.");
		return false;
	}

	TransactionBufferManagerState.SavedGlobalBuffer = GEditor->Trans;
	TransactionBufferManagerState.bIsOverrideActive = true;
	GEditor->Trans = InOverrideBuffer;
	return true;
}

void FTransactionBufferManager::RestoreGlobalBuffer()
{
	check(IsInGameThread());
	check(GEditor);

	if (!TransactionBufferManagerState.bIsOverrideActive)
	{
		UE_LOGF(
			LogAIAssistant, Warning,
			"RestoreGlobalBuffer: No override active (global buffer was never saved)");
		return;
	}

	if (TransactionBufferManagerState.SavedGlobalBuffer
		&& !IsValid(TransactionBufferManagerState.SavedGlobalBuffer))
	{
		UE_LOGF(
			LogAIAssistant, Warning,
			"RestoreGlobalBuffer: Saved global buffer was invalid, keeping current buffer");
		TransactionBufferManagerState.SavedGlobalBuffer = nullptr;
		TransactionBufferManagerState.bIsOverrideActive = false;
		return;
	}
	GEditor->Trans = TransactionBufferManagerState.SavedGlobalBuffer;
	TransactionBufferManagerState.SavedGlobalBuffer = nullptr;
	TransactionBufferManagerState.bIsOverrideActive = false;
}

bool FTransactionBufferManager::IsOverrideActive()
{
	check(IsInGameThread());

	return TransactionBufferManagerState.bIsOverrideActive;
}

TSet<FString> FTransactionBufferManager::GetFilenamesFromUndoStack(const UTransBuffer* UndoStack)
{
	TSet<FString> Filenames;
	if (!UndoStack)
	{
		return Filenames;
	}

	// Only look at transactions that haven't been undone (active transactions).
	const int32 ActiveTransactionCount = UndoStack->GetQueueLength() - UndoStack->GetUndoCount();
	for (int32 Index = 0; Index < ActiveTransactionCount; ++Index)
	{
		const FTransaction* Transaction = UndoStack->GetTransaction(Index);
		if (Transaction)
		{
			// Get all objects that were part of this transaction.
			TArray<UObject*> TransactionObjects;
			Transaction->GetTransactionObjects(TransactionObjects);

			for (UObject* Object : TransactionObjects)
			{
				// Get the package that contains this object.
				UPackage* Package = Object ? Object->GetOutermost() : nullptr;
				if (Package)
				{
					FString PackagePath = Package->GetPathName();
					Filenames.Add(MoveTemp(PackagePath));
				}
			}
		}
	}

	return Filenames;
}

}  // namespace UE::AIAssistant

// Console commands for testing and debugging
namespace
{

void SetTransactionBufferOverrideCommand(const TArray<FString>& Args)
{
	using UE::AIAssistant::FTransactionBufferManager;

	if (Args.Num() < 1)
	{
		UE_LOGF(LogAIAssistant, Warning, "Usage: TransactionBuffer.SetOverride <BufferName>");
		return;
	}
	const FString& BufferName = Args[0];
	UTransBuffer* Buffer = FTransactionBufferManager::GetOrCreateTransactionBuffer(BufferName);
	if (Buffer && FTransactionBufferManager::SetOverrideBuffer(Buffer))
	{
		UE_LOGF(
			LogAIAssistant, Log,
			"Set transaction buffer override to '%ls'", *BufferName);
	}
	else
	{
		UE_LOGF(LogAIAssistant, Error,
			"Failed to set transaction buffer override to '%ls'", *BufferName);
	}
}

void RestoreTransactionBufferCommand()
{
	UE::AIAssistant::FTransactionBufferManager::RestoreGlobalBuffer();
	UE_LOGF(LogAIAssistant, Log, "Restored global transaction buffer");
}

void DestroyTransactionBufferCommand(const TArray<FString>& Args)
{
	using UE::AIAssistant::FTransactionBufferManager;

	if (Args.Num() < 1)
	{
		UE_LOGF(LogAIAssistant, Warning, "Usage: TransactionBuffer.Destroy <BufferName>");
		return;
	}

	FString BufferName(Args[0]);
	if (FTransactionBufferManager::DestroyTransactionBuffer(BufferName))
	{
		UE_LOGF(
			LogAIAssistant, Log,
			"Destroyed transaction buffer '%ls'", *BufferName);
	}
	else
	{
		UE_LOGF(
			LogAIAssistant, Error,
			"Failed to destroy transaction buffer '%ls'", *BufferName);
	}
}

FAutoConsoleCommand CCmdSetTransactionBufferOverride(
	TEXT("TransactionBuffer.SetOverride"),
	TEXT("Set an existing transaction buffer as the active override.)")
	TEXT("Usage: TransactionBuffer.SetOverride <BufferName>"),
	FConsoleCommandWithArgsDelegate::CreateStatic(&SetTransactionBufferOverrideCommand)
);

FAutoConsoleCommand CCmdRestoreTransactionBuffer(
	TEXT("TransactionBuffer.Restore"),
	TEXT("Restore the global transaction buffer."),
	FConsoleCommandDelegate::CreateStatic(&RestoreTransactionBufferCommand)
);

FAutoConsoleCommand CCmdDestroyTransactionBuffer(
	TEXT("TransactionBuffer.Destroy"),
	TEXT("Destroy a transaction buffer. Usage: TransactionBuffer.Destroy <BufferName>"),
	FConsoleCommandWithArgsDelegate::CreateStatic(&DestroyTransactionBufferCommand)
);

} // anonymous namespace
