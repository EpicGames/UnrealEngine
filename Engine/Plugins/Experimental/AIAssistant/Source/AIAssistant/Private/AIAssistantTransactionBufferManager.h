// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Editor/TransBuffer.h"

namespace UE::AIAssistant
{

// Manager for creating and managing multiple independent transaction buffers.
// Allows for the creation of separate undo buffers that temporarily override the global buffer.
class FTransactionBufferManager
{
public:
	// Get an existing transaction buffer by name.
	//
	// @param BufferName Name of the transaction buffer to retrieve
	// @return The transaction buffer if found, nullptr otherwise
	static TObjectPtr<UTransBuffer> GetTransactionBuffer(const FString& BufferName);

	// Create a new transaction buffer with the given name.
	// If a buffer with this name already exists, returns the existing buffer.
	//
	// @param BufferName Unique name for the transaction buffer
	// @return The created or existing transaction buffer
	static TObjectPtr<UTransBuffer> GetOrCreateTransactionBuffer(const FString& BufferName);
	
	// Destroy a transaction buffer by name.
	// Removes it from the registry and releases the root reference.
	// Cannot destroy a buffer that is currently active as an override.
	//
	// @param BufferName Name of the transaction buffer to destroy
	// @return True if the buffer was destroyed, false if not found or currently active
	static bool DestroyTransactionBuffer(const FString& BufferName);

	// Shutdown and cleanup all transaction buffers.
	// Should be called during module shutdown.
	// Restores any active overrides and clears all registered buffers.
	static void Shutdown();

	// Override the global undo buffer with the provided buffer.
	// The original global undo buffer is saved on first call and can be restored at any time.
	//
	// @param OverrideBuffer The transaction buffer to use as override.
	// @return True if the override was set successfully, false otherwise.
	static bool SetOverrideBuffer(UTransactor* OverrideBuffer);

	// Clears the current override and restores the global undo buffer that was saved when
	// SetOverrideBuffer was first called.
	static void RestoreGlobalBuffer();

	// Check if there is currently an override buffer active.
	//
	// @return True if an override buffer is active, false otherwise
	static bool IsOverrideActive();

	// Extract all package path names from an undo stack.
	// Iterates through active transactions (those that haven't been undone) and collects
	// the package paths of all objects that were modified.
	//
	// @param UndoStack The transaction buffer to extract filenames from
	// @return Set of package path names for all modified objects
	static TSet<FString> GetFilenamesFromUndoStack(const UTransBuffer* UndoStack);
};

}  // namespace UE::AIAssistant
