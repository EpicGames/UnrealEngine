// Copyright Epic Games, Inc. All Rights Reserved.

#include "AIAssistantWebApi.h"

#include "Async/Future.h"
#include "Containers/UnrealString.h"
#include "Misc/AssertionMacros.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Object.h"

#define UE_AIASSISTANT_WEB_API_PARENT_OBJECT_NAME TEXT("window")
#define UE_AIASSISTANT_WEB_API_CHILD_OBJECT_NAME TEXT("eda")

namespace UE::AIAssistant
{
	UE_ENUM_METADATA_DEFINE(EMessageRole, UE_AI_ASSISTANT_MESSAGE_ROLE_ENUM);
	UE_ENUM_METADATA_DEFINE(EMessageContentType, UE_AI_ASSISTANT_MESSAGE_CONTENT_TYPE_ENUM);
	UE_ENUM_METADATA_DEFINE(EPendingFileStatus, UE_AI_ASSISTANT_PENDING_FILE_STATUS_ENUM);
	UE_ENUM_METADATA_DEFINE(EConversationUpdateType, UE_AI_ASSISTANT_CONVERSATION_UPDATE_TYPE_ENUM);

	const FString FWebApi::WebApiObjectName =
		UE_AIASSISTANT_WEB_API_PARENT_OBJECT_NAME TEXT(".")
		UE_AIASSISTANT_WEB_API_CHILD_OBJECT_NAME;
	const FString FWebApi::WebApiAvailableFunction = FString::Format(
		TEXT(R"js(
(
  async () => {
    const timeoutInMilliseconds = {0};
    const pollIntervalInMilliseconds = {1};
    const deadline = Date.now() + timeoutInMilliseconds;
    let hasEdaApi = false;
    while (true) {
      hasEdaApi = Object.hasOwn({2}, '{3}');
      if (hasEdaApi || Date.now() > deadline) break;
      await new Promise(resolve => setTimeout(resolve, pollIntervalInMilliseconds));
    }
    return { value: hasEdaApi };
  }
)
)js"),
		{
			2000, // Timeout in milliseconds.
			100, // Poll interval in milliseconds.
			UE_AIASSISTANT_WEB_API_PARENT_OBJECT_NAME,
			UE_AIASSISTANT_WEB_API_CHILD_OBJECT_NAME
		});

	const FString FWebApi::FunctionCallFormatTemplate = TEXT(R"js(
try {
  Promise.resolve({Function}({Arguments})).then(
    (result) => {
      {NotifyHandlerOfResult}
    },
    (error) => {
      {NotifyHandlerOfError}
    });
} catch (error) {
  {NotifyHandlerOfError}
}
)js");

	const FString FWebApi::FunctionCallbackFormatTemplate = TEXT(R"js(
(result) => {
  {NotifyHandlerOfResult}
}
)js");

	const FString FWebApi::ConvertErrorToString =
		TEXT(R"js(error instanceof Error ? error.toString() : error)js");

	FWebApi::FWebApi(
		IWebJavaScriptExecutor& JavaScriptExecutor,
		IWebJavaScriptDelegateBinder& JavaScriptDelegateBinder,
		FSimpleMulticastDelegate& UnbindDelegate) :
		WebJavaScriptExecutor(JavaScriptExecutor),
		WebJavaScriptDelegateBinder(JavaScriptDelegateBinder),
		WebJavaScriptResultDelegate(NewObject<UAIAssistantWebJavaScriptResultDelegate>())
	{
		WebJavaScriptResultDelegate->Bind(*this, UnbindDelegate);
	}

	FWebApi::~FWebApi()
	{
		// Unfortunately on shutdown the delegate can be garbage collected even though a strong
		// reference to it is held.
		if (WebJavaScriptResultDelegate.IsValid())
		{
			WebJavaScriptResultDelegate->Unbind();
		}
	}

	TFuture<TValueOrError<FWebApiBoolResult, FString>> FWebApi::IsAvailable()
	{
		return ExecutionFunctionParseJson<FWebApiBoolResult>(nullptr, *WebApiAvailableFunction);
	}

	void FWebApi::AddMessageToConversation(const FAddMessageToConversationOptions& Options)
	{
		(void)ExecuteFunctionWithJsonArgument(
			*WebApiObjectName, TEXT("addMessageToConversation"), Options);
	}

	TFuture<TValueOrError<void, FString>> FWebApi::CreateConversation()
	{
		return ExecutionFunctionParseJson<void>(*WebApiObjectName, TEXT("createConversation"));
	}

	TFuture<TValueOrError<FConversation, FString>> FWebApi::GetConversation(
		const TOptional<FConversationId>& ConversationId)
	{
		if (ConversationId.IsSet())
		{
			return ExecutionFunctionParseJson<FConversation>(
				*WebApiObjectName, TEXT("getConversation"), *ConversationId);
		}
		else
		{
			return ExecutionFunctionParseJson<FConversation>(
				*WebApiObjectName, TEXT("getConversation"), nullptr);
		}
	}

	TFuture<TValueOrError<FAgentEnvironmentHandle, FString>> FWebApi::AddAgentEnvironment(
		const FAgentEnvironment& AgentEnvironment)
	{
		return ExecutionFunctionParseJson<FAgentEnvironmentHandle>(
			*WebApiObjectName, TEXT("addAgentEnvironment"), AgentEnvironment);
	}

	void FWebApi::SetAgentEnvironment(const FAgentEnvironmentId& AgentEnvironmentId)
	{
		(void)ExecuteFunctionWithJsonArgument(
			*WebApiObjectName, TEXT("setAgentEnvironment"), AgentEnvironmentId);
	}

	void FWebApi::UpdateGlobalLocale(const FString& LocaleString)
	{
		(void)ExecuteFunction(
			*WebApiObjectName, TEXT("updateGlobalLocale"),
			*FString::Printf(TEXT("\"%s\""), *LocaleString));
	}

	TFuture<TValueOrError<FWebApi::FEventCallbackHandle, FString>>
		FWebApi::RegisterOnConversationUpdate(
		FOnConversationUpdateCallback OnConversationUpdate)
	{
		return RegisterEventCallback<FConversationUpdateEvent>(
			TEXT("registerOnConversationUpdate"),
			[OnConversationUpdate = MoveTemp(OnConversationUpdate)](
				const FConversationUpdateEvent& Event) {
				return OnConversationUpdate(Event);
			});
	}

	void FWebApi::UnregisterOnConversationUpdate(const FEventCallbackHandle& Handle)
	{
		UnregisterEventCallback(TEXT("unregisterOnConversationUpdate"), Handle);
	}

	void FWebApi::UpdatePendingFileList(const FUpdatePendingFileListOptions& Options)
	{
		(void)ExecuteFunctionWithJsonArgument(
			*WebApiObjectName, TEXT("updatePendingFileList"), Options);
	}

	TFuture<TValueOrError<FWebApi::FEventCallbackHandle, FString>>
		FWebApi::RegisterOnPendingFileDecision(
		FOnPendingFileDecisionCallback OnPendingFileDecision)
	{
		return RegisterEventCallback<FOnPendingFileDecisionOptions>(
			TEXT("registerOnPendingFileDecision"),
			[OnPendingFileDecision = MoveTemp(OnPendingFileDecision)](
				const FOnPendingFileDecisionOptions& Options) {
				OnPendingFileDecision(Options);
				return UAIAssistantWebJavaScriptResultDelegate::EHandlerStatus::Continue;
			});
	}

	void FWebApi::UnregisterOnPendingFileDecision(const FEventCallbackHandle& Handle)
	{
		UnregisterEventCallback(TEXT("unregisterOnPendingFileDecision"), Handle);
	}

	void FWebApi::UnregisterEventCallback(
		const TCHAR* JsFunctionName,
		const FEventCallbackHandle& Handle)
	{
		auto& [HandlerId, CallbackHandle] = Handle;
		if (!WebJavaScriptResultDelegate.IsValid())
		{
			return;
		}
		(void)ExecuteFunction(
			*WebApiObjectName, JsFunctionName,
			*FString::Printf(TEXT("%s"), *CallbackHandle));
		WebJavaScriptResultDelegate->UnregisterResultHandler(HandlerId);
	}

	FString FWebApi::FormatFunctionCall(
		const TCHAR* InstanceName, const TCHAR* FunctionName, const TCHAR* Arguments,
		const FString& HandlerId) const
	{
		check(FunctionName);
		check(Arguments);
		auto Handlers = FormatResultAndErrorHandlers(HandlerId);
		FString Function =
			InstanceName && *InstanceName != '\0'
			? FString(InstanceName) + TEXT(".") + FunctionName
			: FunctionName;
		return FString::Format(
			*FunctionCallFormatTemplate,
			{
				{ TEXT("Function"), Function },
				{ TEXT("Arguments"), Arguments },
				{ TEXT("NotifyHandlerOfResult"), Handlers.Key },
				{ TEXT("NotifyHandlerOfError"), Handlers.Value },
			});
	}

	FString FWebApi::FormatFunctionCallback(const FString& HandlerId) const
	{
		auto Handlers = FormatResultAndErrorHandlers(HandlerId);
		return FString::Format(
			*FunctionCallbackFormatTemplate,
			FStringFormatNamedArguments {
				{ TEXT("NotifyHandlerOfResult"), Handlers.Key }
			});
	}

	TPair<FString, FString> FWebApi::FormatResultAndErrorHandlers(const FString& HandlerId) const
	{
		if (!HandlerId.IsEmpty())
		{
			FString HandlerFormat =
				WebJavaScriptResultDelegate->FormatJavaScriptHandler(HandlerId);
			return TPair<FString, FString>(
				FString::Format(*HandlerFormat, { TEXT("result"), TEXT("false") }),
				FString::Format(*HandlerFormat, { *ConvertErrorToString, TEXT("true") }));
		}
		return TPair<FString, FString>();
	}

	TFuture<UAIAssistantWebJavaScriptResultDelegate::FResult> FWebApi::ExecuteFunction(
		const TCHAR* InstanceName, const TCHAR* FunctionName, const TCHAR* Arguments)
	{
		auto HandlerIdAndFuture = WebJavaScriptResultDelegate->RegisterResultHandlerForFuture();
		ExecuteAsyncFunction(InstanceName, FunctionName, Arguments, *HandlerIdAndFuture.Key);
		return MoveTemp(HandlerIdAndFuture.Value);
	}

	FString FWebApi::RegisterCallback(
		TFunction<UAIAssistantWebJavaScriptResultDelegate::EHandlerStatus(
			UAIAssistantWebJavaScriptResultDelegate::FResult)> Callback)
	{
		return WebJavaScriptResultDelegate->RegisterResultHandlerForCallback(Callback);
	}

	void FWebApi::ExecuteAsyncFunction(
		const TCHAR* InstanceName, const TCHAR* FunctionName, const TCHAR* Arguments,
		const TCHAR* HandlerId)
	{
		WebJavaScriptExecutor.ExecuteJavaScript(
			FormatFunctionCall(InstanceName, FunctionName, Arguments, HandlerId));
	}

	void FWebApi::BindUObject(const FString& Name, UObject* Object, bool bIsPermanent)
	{
		WebJavaScriptDelegateBinder.BindUObject(Name, Object, bIsPermanent);
	}

	void FWebApi::UnbindUObject(const FString& Name, UObject* Object, bool bIsPermanent)
	{
		WebJavaScriptDelegateBinder.UnbindUObject(Name, Object, bIsPermanent);
		WebJavaScriptResultDelegate.Reset();
	}

}  // namespace UE::AIAssistant
