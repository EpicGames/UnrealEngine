// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Async/Future.h"
#include "Delegates/Delegate.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "Templates/Tuple.h"
#include "Templates/UnrealTemplate.h"
#include "Templates/ValueOrError.h"
#include "UObject/StrongObjectPtr.h"

#include "AIAssistantLog.h"
#include "AIAssistantTypes.h"
#include "AIAssistantWebJavaScriptDelegateBinder.h"
#include "AIAssistantWebJavaScriptExecutor.h"
#include "AIAssistantWebJavaScriptResultDelegate.h"

namespace UE::AIAssistant
{
	// Interface for a subset of the Epic Developer Assistant API.

	// API to communicate with the web assistant.
	class FWebApi : private IWebJavaScriptDelegateBinder
	{
		friend class FWebApiAccessor;

	public:
		FWebApi(
			IWebJavaScriptExecutor& JavaScriptExecutor,
			IWebJavaScriptDelegateBinder& JavaScriptDelegateBinder,
			FSimpleMulticastDelegate& UnbindDelegate);

		// Prevent copy.
		FWebApi(const FWebApi&) = delete;
		FWebApi& operator=(const FWebApi&) = delete;

		virtual ~FWebApi();

		// Whether the web API is available in the current execution environment.
		TFuture<TValueOrError<FWebApiBoolResult, FString>> IsAvailable();

		// Add a message to a conversation.
		void AddMessageToConversation(const FAddMessageToConversationOptions& Options);

		// Create a new conversation.
		TFuture<TValueOrError<void, FString>> CreateConversation();

		// Get a conversation by ID.
		// If no ID is specified, the current conversation should be returned.
		virtual TFuture<TValueOrError<FConversation, FString>> GetConversation(
			const TOptional<FConversationId>& ConversationId);

		// Add an agent environment for the currently logged in user
		// returning the ID. If a matching environment already exists for the
		// user, this should return the existing environment (i.e upsert).
		TFuture<TValueOrError<FAgentEnvironmentHandle, FString>> AddAgentEnvironment(
			const FAgentEnvironment& AgentEnvironment);

		// Set agent environment for the conversational UI.
		void SetAgentEnvironment(const FAgentEnvironmentId& AgentEnvironmentId);

		// Update the global locale state.
		void UpdateGlobalLocale(const FString& LocaleString);

		// Opaque handle to data needed to unregister an event callback.
		using FEventCallbackHandle = TTuple<FString, FString>;

		using FOnConversationUpdateCallback =
			TFunction<UAIAssistantWebJavaScriptResultDelegate::EHandlerStatus(FConversationUpdateEvent)>;

		// Register for conversation updates. To unregister the callback return
		// EHandlerStatus::Unregister or explicitly call UnregisterOnConversationUpdate.
		TFuture<TValueOrError<FEventCallbackHandle, FString>> RegisterOnConversationUpdate(
			FOnConversationUpdateCallback OnConversationUpdate);

		// Unregister conversation update callback using the CallbackRegistration returned from
		// RegisterOnConversationUpdate.
		void UnregisterOnConversationUpdate(const FEventCallbackHandle& Registration);

		// Update the list of pending files modified by tool calls for a conversation.
		// Called by the C++ backend after each tool call to report which files were modified.
		void UpdatePendingFileList(const FUpdatePendingFileListOptions& Options);

		using FOnPendingFileDecisionCallback = TFunction<void(FOnPendingFileDecisionOptions)>;

		// Register a callback for when the user approves or rejects pending file changes.
		TFuture<TValueOrError<FEventCallbackHandle, FString>> RegisterOnPendingFileDecision(
			FOnPendingFileDecisionCallback OnPendingFileDecision);

		// Unregister pending file decision callback using the handle returned from
		// RegisterOnPendingFileDecision.
		void UnregisterOnPendingFileDecision(const FEventCallbackHandle& Registration);

	protected:
		// Format a function call of a member with result handling.
		FString FormatFunctionCall(
			const TCHAR* InstanceName, const TCHAR* FunctionName,
			const TCHAR* Arguments = TEXT(""), const FString& HandlerId = FString()) const;

		// Format a function callback which calls a given handler ID with a result.
		FString FormatFunctionCallback(const FString& HandlerId) const;

		// Format JavaScript for result and error handler function calls.
		TPair<FString, FString> FormatResultAndErrorHandlers(const FString& HandlerId) const;

		// Execute a javascript function getting the result as a JSON encoded string.
		TFuture<UAIAssistantWebJavaScriptResultDelegate::FResult> ExecuteFunction(
			const TCHAR* InstanceName, const TCHAR* FunctionName,
			const TCHAR* Arguments = TEXT(""));

		// Register a callback with Javascript, allowing the given TFunction to be called multiple
		// times from Javascript. This is virtual so that the returned handler ID can be captured
		// in tests.
		virtual FString RegisterCallback(
			TFunction<UAIAssistantWebJavaScriptResultDelegate::EHandlerStatus(
				UAIAssistantWebJavaScriptResultDelegate::FResult)> Callback);

		// Execute a javascript function handling the result with the specified handler.
		// NOTE: This is virtual so tests can capture the function, arguments and handler ID before
		// they're inserted into a script.
		virtual void ExecuteAsyncFunction(
			const TCHAR* InstanceName, const TCHAR* FunctionName, const TCHAR* Arguments,
			const TCHAR* HandlerId);

		// Execute a javascript function converting an argument to JSON.
		template<typename JsonSerializableArgType>
		TFuture<UAIAssistantWebJavaScriptResultDelegate::FResult> ExecuteFunctionWithJsonArgument(
			const TCHAR* InstanceName, const TCHAR* FunctionName,
			const JsonSerializableArgType& Argument)
		{
			return ExecuteFunction(InstanceName, FunctionName, *Argument.ToJson(false));
		}

		// Execute a javascript function converting an empty argument to JSON null.
		TFuture<UAIAssistantWebJavaScriptResultDelegate::FResult> ExecuteFunctionWithJsonArgument(
			const TCHAR* InstanceName, const TCHAR* FunctionName,
			std::nullptr_t NullPtr)
		{
			return ExecuteFunction(InstanceName, FunctionName, TEXT("null"));
		}

		// Create a promise and handler for the execution of a JavaScript function that optionally
		// parses a JSON result value.
		template<typename JsonSerializableReturnType>
		TTuple<
			TSharedPtr<TPromise<TValueOrError<JsonSerializableReturnType, FString>>>,
			TFunction<void(const TFuture<UAIAssistantWebJavaScriptResultDelegate::FResult>&)>>
			CreatePromiseAndHandlerForFunction()
		{
			auto Result =
				MakeShared<TPromise<TValueOrError<JsonSerializableReturnType, FString>>>();
			auto ResultHandler =
				[Result](
					const TFuture<UAIAssistantWebJavaScriptResultDelegate::FResult>&
					ExecutionResultFuture) mutable -> void
				{
					auto ExecutionResult = ExecutionResultFuture.Get();
					const FString& Json = ExecutionResult.Json;
					if (ExecutionResult.bJsonIsError)
					{
						Result->SetValue(MakeError(Json));
					}
					else
					{
						Result->SetValue(ParseJsonIfNotVoid<JsonSerializableReturnType>(Json));
					}
				};
			return MakeTuple(Result, MoveTemp(ResultHandler));
		}

		// Execute a function with no arguments optionally marshalling a return value from JSON or
		// setting an error.
		template<typename JsonSerializableReturnType>
		TFuture<TValueOrError<JsonSerializableReturnType, FString>>
			ExecutionFunctionParseJson(const TCHAR* InstanceName, const TCHAR* FunctionName)
		{
			auto ResultAndHandler = CreatePromiseAndHandlerForFunction<JsonSerializableReturnType>();
			ExecuteFunction(InstanceName, FunctionName).Then(MoveTemp(ResultAndHandler.Value));
			return ResultAndHandler.Key->GetFuture();
		}

		// Execute a function marshalling an argument to JSON and the return value from JSON or
		// setting an error.
		template<typename JsonSerializableReturnType, typename JsonSerializableArgType>
		TFuture<TValueOrError<JsonSerializableReturnType, FString>>
			ExecutionFunctionParseJson(
				const TCHAR* InstanceName,
				const TCHAR* FunctionName,
				const JsonSerializableArgType& Argument)
		{
			auto ResultAndHandler = CreatePromiseAndHandlerForFunction<JsonSerializableReturnType>();
			ExecuteFunctionWithJsonArgument(InstanceName, FunctionName, Argument).Then(
				MoveTemp(ResultAndHandler.Value));
			return ResultAndHandler.Key->GetFuture();
		}

		// Parse a JSON string if JsonSerializableReturnType is not void otherwise return an empty value.
		template<typename JsonSerializableReturnType>
		static TValueOrError<JsonSerializableReturnType, FString> ParseJsonIfNotVoid(const FString& Json)
		{
			JsonSerializableReturnType Parsed;
			if (!Parsed.FromJson(Json))
			{
				return MakeError(FString(TEXT("Failed to parse: ")) + Json);
			}
			return MakeValue(MoveTemp(Parsed));
		}

		// Overload that ignores the supplied JSON string and returns an empty value.
		template<>
		inline TValueOrError<void, FString> ParseJsonIfNotVoid(const FString& UnusedJson)
		{
			return MakeValue();
		}

		// Register a callback for a JavaScript event, parsing the result JSON into
		// OptionsType. InternalCallback returns EHandlerStatus to control whether
		// the callback remains registered.
		template<typename OptionsType>
		TFuture<TValueOrError<FEventCallbackHandle, FString>> RegisterEventCallback(
			const TCHAR* JsFunctionName,
			TFunction<UAIAssistantWebJavaScriptResultDelegate::EHandlerStatus(
				const OptionsType&)> InternalCallback)
		{
			using FResult = UAIAssistantWebJavaScriptResultDelegate::FResult;
			FString HandlerId = RegisterCallback(
				[InternalCallback = MoveTemp(InternalCallback)](FResult Result) {
					OptionsType ParsedOptions;
					if (Result.bJsonIsError || !ParsedOptions.FromJson(Result.Json))
					{
						UE_LOGF(
							LogAIAssistant, Error,
							"Javascript Error running callback: %ls",
							*Result.Json);
						return UAIAssistantWebJavaScriptResultDelegate::
							EHandlerStatus::Unregister;
					}
					return InternalCallback(ParsedOptions);
				});

			TFuture<FResult> Future = ExecuteFunction(
				*WebApiObjectName, JsFunctionName,
				*FormatFunctionCallback(HandlerId));

			TWeakObjectPtr<UAIAssistantWebJavaScriptResultDelegate> WeakDelegate(
				WebJavaScriptResultDelegate.Get());

			return Future.Next(
				[HandlerId, WeakDelegate](const FResult& Result)
					-> TValueOrError<FEventCallbackHandle, FString>
				{
					if (Result.bJsonIsError)
					{
						// The JS registration failed; the handler will never fire.
						// Clean it up to avoid leaking it in the delegate's map.
						if (UAIAssistantWebJavaScriptResultDelegate* Delegate = WeakDelegate.Get())
						{
							Delegate->UnregisterResultHandler(HandlerId);
						}
						return MakeError(Result.Json);
					}
					return MakeValue(FEventCallbackHandle
						{
							HandlerId,
							Result.Json
						});
				});
		}

		// Unregister a previously registered event callback.
		void UnregisterEventCallback(
			const TCHAR* JsFunctionName, const FEventCallbackHandle& Handle);

	private:
		// Call the underlying binder.
		void BindUObject(const FString& Name, UObject* Object, bool bIsPermanent = true) override;

		// Unbind from the underlying binder and remove the reference to
		// WebJavaScriptResultDelegate as it is likely being destroyed.
		void UnbindUObject(const FString& Name, UObject* Object, bool bIsPermanent = true) override;

	protected:
		IWebJavaScriptExecutor& WebJavaScriptExecutor;
		IWebJavaScriptDelegateBinder& WebJavaScriptDelegateBinder;
		TStrongObjectPtr<UAIAssistantWebJavaScriptResultDelegate> WebJavaScriptResultDelegate;

	protected:
		// Fully qualified name of the global object that implements the web API.
		static const FString WebApiObjectName;

		// JavaScript function that determines whether the web API is available.
		static const FString WebApiAvailableFunction;

	private:
		// FString::Format() template that requires:
		// * Function: Fully qualified name of the function to call.
		// * Arguments: String that contains the arguments to pass to the function.
		// * NotifyHandlerOfResult: JavaScript snippet that is called with "result" to handle
		//   the result of the function.
		// * NotifyHandlerOfError: JavaScript snippet that is called with "error" to handle
		//   a function error.
		static const FString FunctionCallFormatTemplate;
		static const FString FunctionCallbackFormatTemplate;

		// JavaScript that converts a variable called "error" into a string or returns the
		// original object.
		static const FString ConvertErrorToString;
	};
}
