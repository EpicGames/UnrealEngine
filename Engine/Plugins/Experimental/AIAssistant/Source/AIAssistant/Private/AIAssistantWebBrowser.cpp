// Copyright Epic Games, Inc. All Rights Reserved.

#include "AIAssistantWebBrowser.h"

#include "Async/UniqueLock.h"
#include "Editor.h"
#include "WebBrowserModule.h"
#include "IWebBrowserWindow.h"
#include "Misc/AssertionMacros.h"
#include "Framework/Application/SlateApplication.h"
#include "Async/TaskGraphInterfaces.h"
#include "Widgets/Docking/SDockTab.h"

#include "AIAssistantConfig.h"
#include "AIAssistantLog.h"
#include "AIAssistantTextMessage.h"

using namespace UE::AIAssistant;


#define LOCTEXT_NAMESPACE "SAIAssistantWebBrowser"


// SWebBrowser::InitialURL() does not seem to load our initial URL. Loading after the SWebBrowser 
// is created works instead. TODO - Investigate.
#define UE_AIA_SET_INITIAL_URL_VIA_WEB_BROWSER_SLATE_ARG 0


void SAIAssistantWebBrowser::Construct(const FArguments& InArgs, const TSharedRef<SDockTab>& InParentTab)
{
	verify(InArgs._CurrentConfig.IsValid());  // Configuration must be set.
	CurrentConfig = InArgs._CurrentConfig;

	auto Config = CurrentConfig->Load();
	
	// Web connection widget.	
	SAssignNew(WebConnectionWidget, SAIAssistantWebConnectionWidget)
		.WhenDisconnectedMessage(
			LOCTEXT("SAIAssistantWebBrowser_NotConnected", "AI Assistant is Not Connected"))
		.OnRequestConnectionState([this]() -> SAIAssistantWebConnectionWidget::EConnectionState
			{
				return GetConnectionState();
			})
		.OnReconnect_Lambda([this, Config]() -> void
			{
				if (WebBrowserWidget.IsValid())
				{
					UE_LOGF(LogAIAssistant, Display, "Reconnecting...");
					// Must reload to (1) clear any error page, (2) make sure the correct page is
					// actually loaded.
					InitializeWebApplication();
					LoadUrl(Config->MainUrl, EOpenBrowserMode::Embedded);
				}
				else
				{
					UE_LOGF(
						LogAIAssistant, Error,
						"Failed to reconnect, web browser widget is not available!");
				}
			});


	// We need to do this mainly to initialize CEF (Chromium Embedded Framework.)
	// NOTE - We have not enabled the WebBrowserWidget for this plugin. If we had, then this would
	// not be necessary, and this would have been taken care of for us.
	IWebBrowserModule& WebBrowserModule = IWebBrowserModule::Get();
	if (!IWebBrowserModule::IsAvailable() || !WebBrowserModule.IsWebModuleAvailable())
	{
		UE_LOGF(
			LogAIAssistant, Error,
			"WebBrowserModule is not available! Can't create WebBrowserWindow");
		return;
	}

	TSharedPtr<IWebBrowserWindow> WebBrowserWindow = WebBrowserModule.GetSingleton()->CreateBrowserWindow(FCreateBrowserWindowSettings());
	if (!WebBrowserWindow.IsValid())
	{
		UE_LOGF(
			LogAIAssistant, Error, 
			"Could not create WebBrowserWindow from WebBrowserModule!");
		return;
	}

	
	// Web Browser.
	SAssignNew(WebBrowserWidget, SWebBrowser, /*passed to SWebBrowser ctr..*/ WebBrowserWindow)
#if UE_AIA_SET_INITIAL_URL_VIA_WEB_BROWSER_SLATE_ARG
		.InitialURL(Config->MainUrl) 
#endif
		.ShowControls(false)
		.ShowErrorMessage(false) // ..suppress showing error pages
		.Visibility(EVisibility::Visible)
		.OnBeforeNavigation_Lambda([this](const FString& Url, const FWebNavigationRequest& Request) -> bool
		{
			UE_LOGF(LogAIAssistant, Display, "Navigating to %ls", *Url);
			// 'Navigation' means an attempt to redirect the current browser window.
			bool bBlockNavigation = false;
			EnsureWebApplication();
			WithWebApplication(
				[this, &Url, &Request](
					TSharedRef<FWebApplication> WebApplication) -> void
				{
					WebApplication->OnBeforeNavigation(Url, Request);
				});
			return bBlockNavigation;
		})
		.OnBeforePopup_Lambda([this](/*no const&*/FString Url, /*no const&*/FString Frame) -> bool
		{
			UE_LOGF(LogAIAssistant, Display, "Opening %ls in external browser", *Url);
			// 'Popup' means an attempt to open a new browser window.
			LoadUrl(Url, EOpenBrowserMode::System);
			return true; // ..means block navigation in THIS browser
		})
		.OnConsoleMessage_Lambda([](
			const FString& Message, const FString& Source, int32 Line,
			EWebBrowserConsoleLogSeverity WebBrowserConsoleLogSeverity) -> void
		{
			// Receives messages from JavaScript.
			switch (WebBrowserConsoleLogSeverity)
			{
			case EWebBrowserConsoleLogSeverity::Error:
				// fall through.
			case EWebBrowserConsoleLogSeverity::Fatal:
				UE_LOGF(LogAIAssistant, Display, "JavaScript Error: '%ls' @ %ls:%d",
					*Message, *Source, Line);
				break;
			case EWebBrowserConsoleLogSeverity::Warning:
				UE_LOGF(LogAIAssistant, Display, "JavaScript Warning: '%ls' @ %ls:%d",
					*Message, *Source, Line);
				break;
			default:
				UE_LOGF(LogAIAssistant, Display, "JavaScript: '%ls' @ %ls:%d",
					*Message, *Source, Line);
				break;
			}
		})
		.OnLoadError_Lambda([this]() -> void
		{
			UE_LOGF(LogAIAssistant, Error, "Load failed");
			if (WebConnectionWidget.IsValid())
			{
				UE_LOGF(LogAIAssistant, Display, "Disconnect web connection widget");
				WebConnectionWidget->Disconnect();
			}
			EnsureWebApplication();
			WithWebApplication(
				[this](TSharedRef<FWebApplication> WebApplication) -> void
				{
					UE_LOGF(LogAIAssistant, Display, "Attempting to reset");
					WebApplication->OnPageLoadError();
				});
		})
		.OnLoadCompleted_Lambda([this]() -> void
		{
			WithWebApplication(
				[this](TSharedRef<FWebApplication> WebApplication) -> void
				{
					WebApplication->OnPageLoadComplete();
				});

			// Set keyboard focus. This makes sure that a blinking cursor will appear in text
			// boxes that are clicked on. Without this, this isn't always the case.
			// NOTE - Make sure we run this in Game Thread, so FSlateApplication::Get() doesn't
			// assert. On Mac, the lambda we're in now will often get called from an AppKit thread.
			{
				TWeakPtr<SWebBrowser> WebBrowserWidgetWeak = WebBrowserWidget;
				AsyncTask(ENamedThreads::GameThread, [WebBrowserWidgetWeak]() -> void
					{
						// Safety checks, since we can rarely end up here during Editor shutdown.
						if (const TSharedPtr<SWebBrowser> WebBrowserWidget = WebBrowserWidgetWeak.Pin();
							WebBrowserWidget.IsValid() && FSlateApplication::IsInitialized())
						{
							FSlateApplication::Get().SetKeyboardFocus(
								WebBrowserWidget, EFocusCause::SetDirectly);
						}
					});
			}
		});
	
	// Widget switcher widget.
	TSharedRef<SWidget> WebBrowserSharedRef = WebBrowserWidget.ToSharedRef();
	TSharedRef<SWidget> WebConnectionSharedRef = WebConnectionWidget.ToSharedRef();
	SAssignNew(WebBrowserOrWebConnectionSwitcherWidget, SWidgetSwitcher)
		.WidgetIndex_Lambda([this, WebBrowserSharedRef, WebConnectionSharedRef]() -> int32
			{
				auto SelectedWidget =
					WebConnectionWidget->GetConnectionState() ==
						SAIAssistantWebConnectionWidget::EConnectionState::Connected
					? WebBrowserSharedRef : WebConnectionSharedRef;
				return WebBrowserOrWebConnectionSwitcherWidget->GetWidgetIndex(SelectedWidget);
			})
		+SWidgetSwitcher::Slot()
			[
				WebBrowserSharedRef
			]
		+SWidgetSwitcher::Slot()
			[
				WebConnectionSharedRef
			];
	
	// Widget tree.
	ChildSlot
		[
			WebBrowserOrWebConnectionSwitcherWidget.ToSharedRef()
		];

	
#if !UE_AIA_SET_INITIAL_URL_VIA_WEB_BROWSER_SLATE_ARG
	InitializeWebApplication();
	LoadUrl(Config->MainUrl, EOpenBrowserMode::Embedded);
#endif
}

/*virtual*/ SAIAssistantWebBrowser::~SAIAssistantWebBrowser()
{
	InitializeWebApplication();

	// In case the web connection widget was trying to reconnect, stop it from reconnecting.
	if (WebConnectionWidget.IsValid())
	{
		WebConnectionWidget->StopReconnecting(); 
	}
}

void SAIAssistantWebBrowser::InitializeWebApplication()
{
	UE::TUniqueLock WebApplicationLock(WebApplicationMutex);
	if (!CurrentConfig->IsExiting())
	{
		MaybeWebApplication = CreateWebApplication(
			*this, *this, CurrentConfig->OnPreExit());
	}
}

/*virtual*/ TSharedPtr<FWebApplication> SAIAssistantWebBrowser::CreateWebApplication(
	IWebJavaScriptExecutor& Executor,
	IWebJavaScriptDelegateBinder& Binder,
	FSimpleMulticastDelegate& OnPreExit)
{
	auto Config = CurrentConfig->GetOrLoad();
	FString DevOptionsRawJson = Config.IsValid() ? Config->DevOptionsRawJson : FString();
	return MakeShared<FWebApplication>(
		FWebApplication::CreateWebApiFactory(Executor, Binder, OnPreExit),
		DevOptionsRawJson);
}

void SAIAssistantWebBrowser::EnsureWebApplication()
{
	UE::TUniqueLock WebApplicationLock(WebApplicationMutex);
	if (!MaybeWebApplication) InitializeWebApplication();
}

void SAIAssistantWebBrowser::WithWebApplication(
	TFunction<void(TSharedRef<FWebApplication>)>&& UsingWebApplication,
	bool bCheckUsingWebApplicationIsCalled) const
{
	UE::TUniqueLock WebApplicationLock(WebApplicationMutex);
	bool bIsExiting = CurrentConfig->IsExiting();
	if (!bIsExiting && MaybeWebApplication.IsValid())
	{
		UsingWebApplication(MaybeWebApplication.ToSharedRef());
	}
	else
	{
		check(!bCheckUsingWebApplicationIsCalled || bIsExiting);
	}
}

void SAIAssistantWebBrowser::LoadUrl(const FString& Url, EOpenBrowserMode Mode) const
{
	switch (Mode)
	{
		case EOpenBrowserMode::System:
		{
			FString ErrorString;
			FPlatformProcess::LaunchURL(*Url, TEXT(""), &ErrorString);
			if (!ErrorString.IsEmpty())
			{
				UE_LOGF(
					LogAIAssistant, Error, 
					"%s() - Could not open URL '%ls' - '%ls'.", __func__, *Url, *ErrorString);
			}
			break;
		}
		case EOpenBrowserMode::Embedded:
		{
			check(WebBrowserWidget);
			WebBrowserWidget->LoadURL(Url);
			break;
		}
	}
}

void SAIAssistantWebBrowser::ExecuteJavaScript(const FString& JavaScript)
{
	check(WebBrowserWidget);
	WebBrowserWidget->ExecuteJavascript(JavaScript);
}

void SAIAssistantWebBrowser::BindUObject(const FString& Name, UObject* Object, bool bIsPermanent)
{
	check(WebBrowserWidget);
	WebBrowserWidget->BindUObject(Name, Object, bIsPermanent);
}

void SAIAssistantWebBrowser::UnbindUObject(const FString& Name, UObject* Object, bool bIsPermanent)
{
	check(WebBrowserWidget);
	WebBrowserWidget->UnbindUObject(Name, Object, bIsPermanent);
}

FReply SAIAssistantWebBrowser::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	return FReply::Handled();
}

FReply SAIAssistantWebBrowser::OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	return FReply::Handled();
}

void SAIAssistantWebBrowser::CreateConversation()
{
	WithWebApplication(
		[](TSharedRef<FWebApplication> WebApplication) -> void
		{
			WebApplication->CreateConversation();
		});
}

void SAIAssistantWebBrowser::AddUserMessageToConversation(
	const FString& VisiblePrompt, const FString& HiddenContext)
{
	WithWebApplication(
		[&VisiblePrompt, &HiddenContext](TSharedRef<FWebApplication> WebApplication) -> void
		{
			WebApplication->AddUserMessageToConversation(
				CreateUserMessage(VisiblePrompt, HiddenContext));
		});
}

SAIAssistantWebConnectionWidget::EConnectionState SAIAssistantWebBrowser::GetConnectionState() const
{
	SAIAssistantWebConnectionWidget::EConnectionState ConnectionState =
		SAIAssistantWebConnectionWidget::EConnectionState::Reconnecting;
	WithWebApplication(
		[&ConnectionState](TSharedRef<FWebApplication> WebApplication) -> void
		{
			switch (WebApplication->GetLoadState())
			{
			case FWebApplication::ELoadState::NotLoaded:
				ConnectionState =
					SAIAssistantWebConnectionWidget::EConnectionState::Reconnecting;
				break;
			case FWebApplication::ELoadState::Error:
				ConnectionState =
					SAIAssistantWebConnectionWidget::EConnectionState::Disconnected;
				break;
			case FWebApplication::ELoadState::Complete:
				ConnectionState =
					SAIAssistantWebConnectionWidget::EConnectionState::Connected;
				break;
			}
		},
		false);
	return ConnectionState;
}

#undef LOCTEXT_NAMESPACE
