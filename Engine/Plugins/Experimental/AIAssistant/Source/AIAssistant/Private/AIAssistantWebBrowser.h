// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/RecursiveMutex.h"
#include "Containers/UnrealString.h"
#include "Misc/Optional.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "SWebBrowser.h"
#include "Widgets/Layout/SWidgetSwitcher.h"

#include "AIAssistantCurrentConfig.h"
#include "AIAssistantWebApplication.h"
#include "AIAssistantWebConnectionWidget.h"
#include "AIAssistantWebJavaScriptDelegateBinder.h"
#include "AIAssistantWebJavaScriptExecutor.h"

//
// SAIAssistantWebBrowser
//
// See all NOTE_JAVASCRIPT_CPP_FUNCTIONS.
// A UObject can be bound to SWebBrowser.
// That will make that Object's UFUNCTIONs available for calling in JavaScript.
// They will be available as "window.ue.namespace.functionname".
// In our case, we may choose to bind the UAIAssistantSubsystem to this web browser, as 'aiassistantsubsystem'.
// So, for example, JavaScript would then be able to call - "window.ue.aiassistantsubsystem.executepythonscriptviajavascript(code)".
// IMPORTANT - Yes, it must be all lowercase, or the call will fail.
//


namespace UE::AIAssistant
{
	// Where to load a URL.
	enum class EOpenBrowserMode
	{
		Embedded,  // Open in the Unreal embedded browser.
		System,    // Open in the system web browser.
	};
}


class SAIAssistantWebBrowser :
	public SCompoundWidget,
	public UE::AIAssistant::IWebJavaScriptDelegateBinder,
	public UE::AIAssistant::IWebJavaScriptExecutor
{
public:
	SLATE_BEGIN_ARGS(SAIAssistantWebBrowser) {}
		// This *must* be set to construct this widget.
		SLATE_ARGUMENT(TSharedPtr<UE::AIAssistant::FCurrentConfig>, CurrentConfig);
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<class SDockTab>& InParentTab);

	virtual ~SAIAssistantWebBrowser();

	/** Loads a URL.
	 * @param Url Url to load.
	 * @param Mode Where to open the URL.
	 */
	void LoadUrl(const FString& Url, UE::AIAssistant::EOpenBrowserMode Mode) const;

	/**
	 * JavaScript string to immediately execute in the web browser.
	 * @param JavaScript
	 */
	virtual void ExecuteJavaScript(const FString& JavaScript) override;

	// See IWebBrowserWindow::BindUObject() / IWebBrowserWindow::UnbindUObject().
	virtual void BindUObject(const FString& Name, UObject* Object, bool bIsPermanent = true) override;
	virtual void UnbindUObject(const FString& Name, UObject* Object, bool bIsPermanent = true) override;

	// Treat all keys as handled by this widget when it has focus. This prevents hotkeys from firing when users chat with the assistant.
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	virtual FReply OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;


	// High level conversation API.

	// Asynchronously create a new conversation.
	void CreateConversation();

	// Add a message to the existing conversation.
	// If a new conversation is being currently being created, clear enqueued messages and
	// enqueue the specified message.
	void AddUserMessageToConversation(
		const FString& VisiblePrompt, const FString& HiddenContext = FString());

protected:
	// Create the WebApplication for this browser. Override to provide a custom
	// WebApplication subclass (e.g. for testing).
	virtual TSharedPtr<UE::AIAssistant::FWebApplication> CreateWebApplication(
		UE::AIAssistant::IWebJavaScriptExecutor& Executor,
		UE::AIAssistant::IWebJavaScriptDelegateBinder& Binder,
		FSimpleMulticastDelegate& OnPreExit);

private:
	// Initialize this object to its initial state.
	// This is used on construction and when reloading the page after the system has gone offline.
	void InitializeWebApplication();

	// Initialize the web application if it's not already initialized.
	void EnsureWebApplication();

	// Get a reference to the current web application, if it's valid.
	// If the reference is not valid and the editor is not shutting down,
	// assert / check on if bCheckUsingWebApplicationIsCalled is true.
	// When the editor is shutting down UsingWebApplication will not be called.
	void WithWebApplication(
		TFunction<void(TSharedRef<UE::AIAssistant::FWebApplication>)>&& UsingWebApplication,
		bool bCheckUsingWebApplicationIsCalled = true) const;

	// Get the current connection state.
	SAIAssistantWebConnectionWidget::EConnectionState GetConnectionState() const;

	// Widgets.
	TSharedPtr<SWebBrowser> WebBrowserWidget;
	TSharedPtr<SAIAssistantWebConnectionWidget> WebConnectionWidget;
	// Switches between web browser and connection widgets.
	TSharedPtr<SWidgetSwitcher> WebBrowserOrWebConnectionSwitcherWidget;

	// Configuration.
	TSharedPtr<UE::AIAssistant::FCurrentConfig> CurrentConfig;

	// Guards MaybeWebApplication.
	mutable UE::FRecursiveMutex WebApplicationMutex;
	// Web application used by the widget.
	TSharedPtr<UE::AIAssistant::FWebApplication> MaybeWebApplication;
};
