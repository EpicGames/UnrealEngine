// Copyright Epic Games, Inc. All Rights Reserved.


#include "AIAssistantWebConnectionWidget.h"

#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Images/SThrobber.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SBoxPanel.h"
#include "Styling/StyleColors.h"
#include "HAL/PlatformTime.h"


#define LOCTEXT_NAMESPACE "SAIAssistantWebConnectionWidget"


static constexpr float ReconnectingTickerCheckPeriodSeconds = 5.0;


//
// SAIAssistantWebConnectionWidget
//


void SAIAssistantWebConnectionWidget::Construct(const FArguments& InArgs)
{
	bUseReconnectingTicker = InArgs._bUseReconnectingTicker;
	OnConnectedDelegate = InArgs._OnConnected;
	OnDisconnectedDelegate = InArgs._OnDisconnected;
	OnReconnectDelegate = InArgs._OnReconnect;
	OnRequestConnectionStateFunction = InArgs._OnRequestConnectionState;	

	
	// Makes Reconnect button enabled at start.
	if (LastReconnectAttemptSeconds <= 0.0)
	{
		LastReconnectAttemptSeconds = FPlatformTime::Seconds() - ReconnectingTickerCheckPeriodSeconds;
	}

	
	auto IsReconnectWaiting = [this]() -> bool
	{
		return ((FPlatformTime::Seconds() - LastReconnectAttemptSeconds) < ReconnectingTickerCheckPeriodSeconds);
	};
	
		
	ChildSlot
	[
		SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			.Padding(8.0f)
			[
				SNew(SVerticalBox)
					+ SVerticalBox::Slot()
						.VAlign(VAlign_Center)
						.FillHeight(1.0f)
						[
							SNew(SBox)
								.HAlign(HAlign_Center)
								.VAlign(VAlign_Center)
								.MaxDesiredWidth(800.0f)
								[
									SNew(SBorder)
										.Clipping(EWidgetClipping::ClipToBounds)
										.BorderImage(FAppStyle::GetBrush("HomeScreen.NoInternet.Border"))
										.Padding(16.0f)
										[
											SNew(SVerticalBox)
												+ SVerticalBox::Slot()
													.AutoHeight()
													.Padding(0.0f, 0.0f, 0.0f, 16.0f)
													.HAlign(HAlign_Left)
													[
														SNew(SBox)
															.WidthOverride(24.0f)
															.HeightOverride(24.0f)
															[
																SNew(SWidgetSwitcher)
																	.WidgetIndex_Lambda([this, IsReconnectWaiting]() -> int32
																		{
																			return (ConnectionState == EConnectionState::Reconnecting || IsReconnectWaiting() ? 1 : 0);
																		})
																	+ SWidgetSwitcher::Slot()
																		[
																			SNew(SImage)
																				.Image(FAppStyle::GetBrush("HomeScreen.NoInternet.Icon"))
																		]
																	+ SWidgetSwitcher::Slot()
																		[
																			SNew(SCircularThrobber)
																				.NumPieces(8)
																				.Period(1.2f)
																				.Radius(12.0f)
																		]
															]
													]
													+ SVerticalBox::Slot()
														.FillHeight(0.5f)
														.Padding(0.0f, 0.0f, 0.0f, 16.0f)
														[
															SNew(STextBlock)
																.Font(FAppStyle::GetFontStyle("BoldFont"))
																.Text(InArgs._WhenDisconnectedMessage)
																.ColorAndOpacity(FStyleColors::White)
														]
													+ SVerticalBox::Slot()
														.FillHeight(1.0f)
														[
															SNew(SButton)
																.HAlign(HAlign_Center)
																.ButtonStyle(FAppStyle::Get(), "HomeScreen.MyFolderButton")
																.Text(LOCTEXT("AIAssistantWebConnection_Reconnect", "Reconnect"))
																.IsEnabled_Lambda([this, IsReconnectWaiting]() -> bool
																	{
																		return ((ConnectionState == EConnectionState::Disconnected) && !IsReconnectWaiting());
																	})
																.OnClicked_Lambda([this]() -> FReply
																	{
																		StartReconnecting();
																		
																		return FReply::Handled();
																	})
														]
										]
								]
						]
			]
	];
}


SAIAssistantWebConnectionWidget::EConnectionState SAIAssistantWebConnectionWidget::GetConnectionState() const
{
	return ConnectionState;
}


void SAIAssistantWebConnectionWidget::Disconnect()
{
	if (ConnectionState == EConnectionState::Disconnected)
	{
		return;
	}


	StopReconnecting();
	
	ConnectionState = EConnectionState::Disconnected;
	OnDisconnectedDelegate.ExecuteIfBound();
}


void SAIAssistantWebConnectionWidget::StartReconnecting()
{
	if (ConnectionState == EConnectionState::Connected || ConnectionState == EConnectionState::Reconnecting)
	{
		return;	
	}


	LastReconnectAttemptSeconds = FPlatformTime::Seconds();

	if (bUseReconnectingTicker)
	{
		ReconnectingTicker = FTSTicker::GetCoreTicker().AddTicker(
			FTickerDelegate::CreateSP(this, &SAIAssistantWebConnectionWidget::ReceiveReconnectingTickerUpdate),
			ReconnectingTickerCheckPeriodSeconds);
	}
	
	ConnectionState = EConnectionState::Reconnecting;
	OnReconnectDelegate.ExecuteIfBound();
}


void SAIAssistantWebConnectionWidget::StopReconnecting()
{
	if (ConnectionState != EConnectionState::Reconnecting)
	{
		return;
	}

	
	if (ReconnectingTicker.IsValid())
	{
		FTSTicker::RemoveTicker(ReconnectingTicker);
		ReconnectingTicker.Reset();
	}

	// Don't set new state, don't call a delegate.
}


void SAIAssistantWebConnectionWidget::UpdateReconnecting()
{
	if (ConnectionState == EConnectionState::Connected || ConnectionState == EConnectionState::Disconnected)
	{
		return;	
	}
	
	if (const EConnectionState RequestedConnectionState = (OnRequestConnectionStateFunction ? OnRequestConnectionStateFunction() : EConnectionState::Connected);
		RequestedConnectionState == EConnectionState::Connected)
	{
		StopReconnecting();
		
		ConnectionState = EConnectionState::Connected;
		OnConnectedDelegate.ExecuteIfBound();
	}
	else if (RequestedConnectionState == EConnectionState::Disconnected)
	{
		StopReconnecting();
	
		ConnectionState = EConnectionState::Disconnected;
		OnDisconnectedDelegate.ExecuteIfBound();
	}
}


bool SAIAssistantWebConnectionWidget::ReceiveReconnectingTickerUpdate(float DeltaTime)
{
	UpdateReconnecting();

	return (ConnectionState == EConnectionState::Reconnecting);
}


#undef LOCTEXT_NAMESPACE
