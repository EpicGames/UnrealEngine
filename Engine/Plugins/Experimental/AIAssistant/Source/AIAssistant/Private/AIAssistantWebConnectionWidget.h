// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once


#include "Widgets/SCompoundWidget.h"
#include "Containers/Ticker.h"
#include "HAL/CriticalSection.h"


#define LOCTEXT_NAMESPACE "SAIAssistantWebConnectionWidget"


class SButton;


class SAIAssistantWebConnectionWidget :
	public SCompoundWidget
{
public:


	DECLARE_DELEGATE(FOnConnected);
	DECLARE_DELEGATE(FOnDisconnected);
	DECLARE_DELEGATE(FOnReconnect);


	enum class EConnectionState : uint8
	{
		Connected,
		Disconnected,
		Reconnecting
	};


	using FOnRequestConnectionState = TFunction<EConnectionState()>;
	
	
	SLATE_BEGIN_ARGS(SAIAssistantWebConnectionWidget) :
		_WhenDisconnectedMessage(LOCTEXT("AIAssistantWebConnection_NoInternetConnection", "No Internet Connection")),
		_bUseReconnectingTicker(true)
		{
		}
		SLATE_ARGUMENT(FText, WhenDisconnectedMessage)
		SLATE_ARGUMENT(bool, bUseReconnectingTicker)
		SLATE_ARGUMENT(FOnRequestConnectionState, OnRequestConnectionState)
		SLATE_EVENT(FOnConnected, OnConnected)
		SLATE_EVENT(FOnDisconnected, OnDisconnected)
		SLATE_EVENT(FOnReconnect, OnReconnect)
	SLATE_END_ARGS()


	void Construct(const FArguments& InArgs);


	/**
	 * @return The current connection state.
	 */
	EConnectionState GetConnectionState() const;

	/**
	 * Force disconnected state.
	 */
	void Disconnect();

	/**
	 * Start attempt to acquire connected state.
	 */
	void StartReconnecting();
	
	/**
	 * Stop attempting to acquire connected state.
	 */
	void StopReconnecting();

	/**
	 * Update to check reconnection. Should probably only call this if not using the reconnecting ticker, which calls this automatically.
	 */
	void UpdateReconnecting();
	
	
private:

	
	bool ReceiveReconnectingTickerUpdate(float DeltaTime);
	

	EConnectionState ConnectionState = EConnectionState::Connected;
	FOnConnected OnConnectedDelegate;
	FOnDisconnected OnDisconnectedDelegate;
	FOnReconnect OnReconnectDelegate;
	bool bUseReconnectingTicker = true;
	FTSTicker::FDelegateHandle ReconnectingTicker;
	FOnRequestConnectionState OnRequestConnectionStateFunction;
	double LastReconnectAttemptSeconds = 0.0;
};


#undef LOCTEXT_NAMESPACE
