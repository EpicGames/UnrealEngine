// Copyright Epic Games, Inc. All Rights Reserved.

#include "AIAssistantWebJavaScriptDelegateBinder.h"

#include "Containers/UnrealString.h"
#include "UObject/Object.h"

namespace UE::AIAssistant
{
	FScopedWebJavaScriptDelegateBinder::FScopedWebJavaScriptDelegateBinder(
		IWebJavaScriptDelegateBinder& Binder, const FString& Name, UObject* Object,
		bool bIsPermanent) :
		WebJavaScriptDelegateBinder(Binder),
		BoundObjectName(Name),
		BoundObject(Object),
		bIsPermanentBinding(bIsPermanent)
	{
		WebJavaScriptDelegateBinder.BindUObject(
			BoundObjectName, BoundObject, bIsPermanentBinding);
	}

	FScopedWebJavaScriptDelegateBinder::~FScopedWebJavaScriptDelegateBinder()
	{
		WebJavaScriptDelegateBinder.UnbindUObject(
			BoundObjectName, BoundObject, bIsPermanentBinding);
	}
}