// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/UnrealString.h"
#include "UObject/Object.h"

namespace UE::AIAssistant
{
	// Provides a way to expose a UObject (i.e UFUNCTION) to a JavaScript execution environment
	// (e.g a browser).
	class IWebJavaScriptDelegateBinder
	{
	public:
		virtual ~IWebJavaScriptDelegateBinder() = default;

		// Make Object with its properties (UPROPERTY) and functions (UFUNCTION) accessible from
		// JavaScript as the name "window.ue.{Name}".
		// 
		// For more information, see IWebBrowserWindow::BindUObject().
		virtual void BindUObject(const FString& Name, UObject* Object, bool bIsPermanent = true) = 0;

		// Unbind an object from JavaScript.
		//
		// See IWebBrowserWindow::UnbindUObject().
		virtual void UnbindUObject(const FString& Name, UObject* Object, bool bIsPermanent = true) = 0;
	};

	// RAII container for a IWebJavaScriptDelegateBinder that binds a UObject on construction and
	// unbinds on destruction.
	class FScopedWebJavaScriptDelegateBinder
	{
	public:
		FScopedWebJavaScriptDelegateBinder(
			IWebJavaScriptDelegateBinder& Binder, const FString& Name, UObject* Object,
			bool bIsPermanent = true);

		// Prevent copy.
		FScopedWebJavaScriptDelegateBinder(const FScopedWebJavaScriptDelegateBinder&) = delete;
		FScopedWebJavaScriptDelegateBinder& operator=(
			const FScopedWebJavaScriptDelegateBinder&) = delete;

		~FScopedWebJavaScriptDelegateBinder();

	private:
		IWebJavaScriptDelegateBinder& WebJavaScriptDelegateBinder;
		FString BoundObjectName;
		UObject* BoundObject;
		bool bIsPermanentBinding;
	};
}