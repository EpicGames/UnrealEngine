// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/UnrealString.h"

namespace UE::AIAssistant
{
	// Executes JavaScript in a web browser.
	class IWebJavaScriptExecutor
	{
	public:
		virtual ~IWebJavaScriptExecutor() = default;

		// Execute JavaScript in a web browser without getting the result.
		// Use UAIAssistantWebJavaScriptResultDelegate to get the result of a JavaScript execution
		// in C++.
		virtual void ExecuteJavaScript(const FString& JavaScriptText) = 0;
	};
}