// Copyright Epic Games, Inc. All Rights Reserved.

#include "AIAssistantWebJavaScriptResultDelegate.h"

#include "Async/UniqueLock.h"
#include "Templates/SharedPointer.h"

// Name of this object when registered with the JavaScript binder.
// NOTE: This is lower case as FCEFJSScripting::GetBindingName() will silently change the name
// to lower case if FCEFJSScripting::bJSBindingToLoweringEnabled is true (the default).
const FString UAIAssistantWebJavaScriptResultDelegate::BaseName(TEXT("aiassistantresultdelegate"));
// Result set when a promise is canceled.
const FString UAIAssistantWebJavaScriptResultDelegate::CanceledError(
	TEXT(R"json("canceled")json"));

void UAIAssistantWebJavaScriptResultDelegate::Bind(
	UE::AIAssistant::IWebJavaScriptDelegateBinder& WebJavaScriptDelegateBinder,
	FSimpleMulticastDelegate& UnbindDelegate)
{
	ScopedWebJavaScriptDelegateBinder.Emplace(
		WebJavaScriptDelegateBinder, GetName(), this, true /* bIsPermanent */);
	OnPreExitDelegateHandle = UE::ToolsetRegistry::FDelegateHandleRaii::Create(
		UnbindDelegate,
		UnbindDelegate.AddUObject(
			this, &UAIAssistantWebJavaScriptResultDelegate::Unbind));
}

void UAIAssistantWebJavaScriptResultDelegate::Unbind()
{
	ScopedWebJavaScriptDelegateBinder.Reset();
	CompleteAllPendingPromises();
	OnPreExitDelegateHandle.Reset();
}

FString UAIAssistantWebJavaScriptResultDelegate::CreateHandlerId()
{
	return FGuid::NewGuid().ToString(EGuidFormats::DigitsLower);
}

void UAIAssistantWebJavaScriptResultDelegate::RegisterResultHandlerWithId(
	FResultHandler&& Handler, const FString& HandlerId)
{
	UE::TUniqueLock Lock(ResultHandlersByIdLock);
	ResultHandlersById.Emplace(HandlerId, MakeShared<FResultHandler>(MoveTemp(Handler)));
}

void UAIAssistantWebJavaScriptResultDelegate::UnregisterResultHandler(const FString& HandlerId)
{
	UE::TUniqueLock Lock(ResultHandlersByIdLock);
	ResultHandlersById.Remove(HandlerId);
}

FString UAIAssistantWebJavaScriptResultDelegate::RegisterResultHandler(FResultHandler&& Handler)
{
	FString HandlerId = CreateHandlerId();
	RegisterResultHandlerWithId(MoveTemp(Handler), HandlerId);
	return HandlerId;
}

TPair<FString, TFuture<UAIAssistantWebJavaScriptResultDelegate::FResult>>
UAIAssistantWebJavaScriptResultDelegate::RegisterResultHandlerForFuture()
{
	FString HandlerId = CreateHandlerId();
	TFuture<FResult> Future = CreatePromiseAndGetFuture(HandlerId);
	RegisterResultHandlerWithId(
		[this](FResultHandlerContext&& ResultHandlerContext) mutable -> EHandlerStatus
		{
			TryCompletePendingPromise(MoveTemp(ResultHandlerContext));
			return EHandlerStatus::Unregister;
		},
		HandlerId);
	return TPair<FString, TFuture<FResult>>(MoveTemp(HandlerId), MoveTemp(Future));
}

FString UAIAssistantWebJavaScriptResultDelegate::RegisterResultHandlerForCallback(
	TFunction<EHandlerStatus(FResult)> Callback)
{
	FString HandlerId = CreateHandlerId();
	RegisterResultHandlerWithId(
		[Callback=MoveTemp(Callback)](FResultHandlerContext ResultHandlerContext) mutable ->
			UAIAssistantWebJavaScriptResultDelegate::EHandlerStatus
		{
			return Callback(MoveTemp(ResultHandlerContext));
		},
		HandlerId);
	return HandlerId;
}

void UAIAssistantWebJavaScriptResultDelegate::HandleResult(
	const FString& HandlerId, const FString& ResultJson, bool bResultJsonIsError)
{
	TSharedPtr<FResultHandler> Handler;
	{
		UE::TUniqueLock Lock(ResultHandlersByIdLock);
		TSharedPtr<FResultHandler>* HandlerPtrPtr = ResultHandlersById.Find(HandlerId);
		if (!HandlerPtrPtr) return;
		Handler = *HandlerPtrPtr;
	}
	EHandlerStatus Result =
		(*Handler)(FResultHandlerContext{ { ResultJson, bResultJsonIsError }, HandlerId });
	if (Result == EHandlerStatus::Unregister)
	{
		UnregisterResultHandler(HandlerId);
	}
}

FString UAIAssistantWebJavaScriptResultDelegate::FormatJavaScriptHandler(const FString& HandlerId) const
{
	return FString::Printf(
		TEXT(R"js(window.ue.%s.%s("%s", JSON.stringify({0}), {1});)js"),
		*GetName(),
		*GET_FUNCTION_NAME_CHECKED(
			UAIAssistantWebJavaScriptResultDelegate, HandleResult).ToString().ToLower(),
		*HandlerId);
}

const FString& UAIAssistantWebJavaScriptResultDelegate::GetName() const
{
	UE::TUniqueLock Lock(NameLock);
	if (Name.IsEmpty())
	{
		Name = FString::Printf(
			TEXT("%s_%s"), *BaseName,
			*FGuid::NewGuid().ToString(EGuidFormats::DigitsLower));
	}
	return Name;
}

void UAIAssistantWebJavaScriptResultDelegate::BeginDestroy()
{
	Super::BeginDestroy();
	Unbind();
}

void UAIAssistantWebJavaScriptResultDelegate::CompleteAllPendingPromises()
{
	UE::TUniqueLock Lock(PendingPromisesByIdLock);
	for (auto& HandlerIdAndPromise : PendingPromisesById)
	{
		HandlerIdAndPromise.Value.SetValue(FResult{ CanceledError, true });
	}
	PendingPromisesById.Empty();
}

TFuture<UAIAssistantWebJavaScriptResultDelegate::FResult>
UAIAssistantWebJavaScriptResultDelegate::CreatePromiseAndGetFuture(
	const FString& HandlerId)
{
	UE::TUniqueLock Lock(PendingPromisesByIdLock);
	return PendingPromisesById.Add(HandlerId).GetFuture();
}

void UAIAssistantWebJavaScriptResultDelegate::TryCompletePendingPromise(
	FResultHandlerContext&& ResultHandlerContext)
{
	bool bFulfillPromise = false;
	TPromise<FResult> PromiseToFulfill;
	{
		UE::TUniqueLock Lock(PendingPromisesByIdLock);
		FString HandlerId = ResultHandlerContext.HandlerId;
		auto* Promise = PendingPromisesById.Find(HandlerId);
		if (Promise)
		{
			PromiseToFulfill = MoveTemp(*Promise);
			bFulfillPromise = true;
			PendingPromisesById.Remove(HandlerId);
		}
	}
	if (bFulfillPromise)
	{
		// NOTE: If PendingPromisesByIdLock is held when this is set any calls to register
		// handlers will attempt to acquire PendingPromisesByIdLock and deadlock.
		PromiseToFulfill.SetValue(MoveTemp(ResultHandlerContext));
	}
}