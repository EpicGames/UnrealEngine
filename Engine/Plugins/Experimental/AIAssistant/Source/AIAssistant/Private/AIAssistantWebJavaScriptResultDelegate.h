// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Async/Mutex.h"
#include "Async/Future.h"
#include "Containers/UnrealString.h"
#include "Misc/CoreDelegates.h"
#include "Misc/Guid.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "Templates/Tuple.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Object.h"

#include "ToolsetRegistry/DelegateHandle.h"
#include "AIAssistantWebJavaScriptDelegateBinder.h"

#include "AIAssistantWebJavaScriptResultDelegate.generated.h"

namespace UE::AIAssistant
{
	class FWebJavaScriptResultDelegateAccessor;
}

// Routes a JavaScript function call to a C++ handler (ResultHandler).
//
// This is registered with a JavaScript execution environment (e.g a web browser) using Bind().
// After binding, C++ result handlers can be registered to handle callbacks using
// RegisterResultHandler().
UCLASS()
class UAIAssistantWebJavaScriptResultDelegate : public UObject
{
	// Expose this class to a test accessor object.
	friend class UE::AIAssistant::FWebJavaScriptResultDelegateAccessor;

	GENERATED_BODY()

public:
	// Result of a JavaScript execution.
	struct FResult
	{
		// JSON representation of a JavaScript function's result or error object.
		FString Json;
		// Whether an error occurred, i.e ResultJson is represents an error object.
		bool bJsonIsError;
	};

	// Context passed to a result handler.
	struct FResultHandlerContext : public FResult
	{
		// ID of the handler, this is used internally by
		// UAIAssistantWebJavaScriptResultDelegate.
		FString HandlerId;
	};

	// Whether or not the handler should remain registered or be unregistered after running.
	enum class EHandlerStatus
	{
		Continue,
		Unregister
	};

	// Handles a JavaScript execution result.
	// This functor should return EHandlerStatus::Unregister to unregister the handler or
	// EHandlerStatus::Continue to leave the handler registered for future callbacks.
	using FResultHandler = TFunction<EHandlerStatus(FResultHandlerContext&&)>;

public:
	// Bind this object to a JavaScript execution environment as Name ("aiassistantresultdelegate").
	void Bind(
		UE::AIAssistant::IWebJavaScriptDelegateBinder& WebJavaScriptDelegateBinder,
		FSimpleMulticastDelegate& UnbindDelegate = FCoreDelegates::OnPreExit);

	// Unregister this object from the previous JavaScript delegate binder.
	// NOTE: This must be called before the bound IWebJavaScriptDelegateBinder is destroyed. It's not
	// possible to rely upon automatic object lifetimes as this is a UObject and therefore can be
	// garbage collected way far later than all references to it are dropped.
	void Unbind();

	// Register a handler for a JavaScript execution returning the ID that should be passed to this
	// delegate.
	FString RegisterResultHandler(FResultHandler&& Handler);

	// Register a handler that completes a future when it's executed.
	// This handler will always only be executed once.
	TPair<FString, TFuture<FResult>> RegisterResultHandlerForFuture();

	// Register a callback that may be called multiple times. Return EHandlerStatus::Unregister
	// from the callback to unregister the handler, or explicitly call UnregisterResultHandler.
	FString RegisterResultHandlerForCallback(TFunction<EHandlerStatus(FResult)> Callback);

	// Unregister a previously registered handler with the specified ID.
	void UnregisterResultHandler(const FString& HandlerId);

	// Generate a FString::Format() format string to call the JavaScript function associated with
	// the handler using the specified ID. The returned format string has two arguments "{0}"
	// that expects to accept an object that can be serialized as JSON to pass to the
	// ResultHandler and "{1}" which should be truthy if the result is an error, falsey otherwise.
	FString FormatJavaScriptHandler(const FString& HandlerId) const;

	// Get the name of the delegate.
	const FString& GetName() const;

protected:
	// Handle the result of an execution.
	// HandlerId is the unique ID of the execution, ResultJson is the result of the execution or
	// an error as JSON and bResultJsonIsError is true if ResultJson is an error.
	UFUNCTION(BlueprintCallable, Category = "JavaScript", meta = (BlueprintInternalUseOnly))
	void HandleResult(
		const FString& HandlerId, const FString& ResultJson, bool bResultJsonIsError);

	// Called when the object starts being destroyed, this unbinds and cancels all pending
	// promises.
	void BeginDestroy() override;

private:
	// Register a handler for a JavaScript execution with the specified ID.
	void RegisterResultHandlerWithId(FResultHandler&& Handler, const FString& HandlerId);

	// Complete all pending promises.
	void CompleteAllPendingPromises();

	// Create a promise for a handler and get a future from the promise.
	TFuture<FResult> CreatePromiseAndGetFuture(const FString& HandlerId);

	// Complete a pending promise if the handler associated with the result is still available.
	void TryCompletePendingPromise(FResultHandlerContext&& ResultContext);

private:
	mutable UE::FMutex NameLock; // Guards Name
	// Name of the delegate that is lazily initialized to avoid exposing this as a UObject
	// property.
	mutable FString Name;
	UE::FMutex ResultHandlersByIdLock;  // Guards ResultHandlersById
	TMap<FString, TSharedPtr<FResultHandler>> ResultHandlersById;
	TOptional<UE::AIAssistant::FScopedWebJavaScriptDelegateBinder>
		ScopedWebJavaScriptDelegateBinder;
	UE::FMutex PendingPromisesByIdLock; // Guards PendingPromisesById
	TMap<FString, TPromise<FResult>> PendingPromisesById;

	UE::ToolsetRegistry::FDelegateHandleRaii OnPreExitDelegateHandle;

private:
	// Create a handler ID.
	static FString CreateHandlerId();

public:
	// Base name of this object when registered with the JavaScript binder.
	static const FString BaseName;
	// JSON result when a promise / future is canceled.
	static const FString CanceledError;
};