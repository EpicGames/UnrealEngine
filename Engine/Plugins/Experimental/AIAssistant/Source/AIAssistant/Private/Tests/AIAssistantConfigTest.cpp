// Copyright Epic Games, Inc. All Rights Reserved.

#include "Math/UnrealMathUtility.h"
#include "Misc/AssertionMacros.h"
#include "Misc/AutomationTest.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"

#include "AIAssistantConfig.h"
#include "Tests/AIAssistantTemporaryDirectory.h"
#include "Tests/AIAssistantTestFlags.h"

#if WITH_DEV_AUTOMATION_TESTS

using namespace UE::AIAssistant;

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantConfigTestHasDefaultSearchDirectories,
	"AI.Assistant.Config.HasDefaultSearchDirectories",
	AIAssistantTest::Flags);

bool FAIAssistantConfigTestHasDefaultSearchDirectories::RunTest(const FString& UnusedParameters)
{
	const auto DefaultSearchDirectories = FConfig::GetDefaultSearchDirectories();
	(void)TestNotEqual(
		TEXT("NumDefaultSearchDirectories"), DefaultSearchDirectories.Num(), 0);
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantConfigTestFindConfigFileExists,
	"AI.Assistant.Config.FindConfigFileExists",
	AIAssistantTest::Flags);

bool FAIAssistantConfigTestFindConfigFileExists::RunTest(const FString& UnusedParameters)
{
	const FTemporaryDirectory TemporaryDirectoryUnused;  // Should be ignored.
	const FTemporaryDirectory TemporaryDirectoryWithConfig;
	const FString ConfigFilename =
		FPaths::Combine(*TemporaryDirectoryWithConfig, FConfig::DefaultFilename);
	TestTrue(TEXT("WriteConfigFile"), FFileHelper::SaveStringToFile(TEXT(""), *ConfigFilename));
	(void)TestEqual(
		TEXT("FindConfigFileExists"),
		FConfig::FindConfigFile({ *TemporaryDirectoryUnused, *TemporaryDirectoryWithConfig }),
		ConfigFilename);
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantConfigTestFindConfigFileMissing,
	"AI.Assistant.Config.FindConfigFileMissing",
	AIAssistantTest::Flags);

bool FAIAssistantConfigTestFindConfigFileMissing::RunTest(const FString& UnusedParameters)
{
	const FTemporaryDirectory TemporaryDirectory;
	(void)TestEqual(
		TEXT("FindConfigFileMissing"),
		FConfig::FindConfigFile({ *TemporaryDirectory }),
		"");
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantConfigTestLoadDefault,
	"AI.Assistant.Config.LoadDefault",
	AIAssistantTest::Flags);

bool FAIAssistantConfigTestLoadDefault::RunTest(const FString& UnusedParameters)
{
	const FConfig Config = FConfig::Load("");
	(void)TestEqual("main_url", Config.MainUrl, FConfig::DefaultMainUrl);
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantConfigTestLoadMainUrl,
	"AI.Assistant.Config.LoadMainUrl",
	AIAssistantTest::Flags);

bool FAIAssistantConfigTestLoadMainUrl::RunTest(const FString& UnusedParameters)
{
	const FTemporaryDirectory TemporaryDirectory;
	const FString ConfigFilename(FPaths::Combine(*TemporaryDirectory, FConfig::DefaultFilename));
	verify(
		FFileHelper::SaveStringToFile(
			FString(TEXT(R"json({"main_url": "https://localhost/assistant"})json")),
			*ConfigFilename));
	FConfig Config = FConfig::Load(ConfigFilename);
	(void)TestEqual(TEXT("main_url"), Config.MainUrl, "https://localhost/assistant");
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantConfigTestIgnoresAdditionalFields,
	"AI.Assistant.Config.LoadIgnoresAdditionalFields",
	AIAssistantTest::Flags);

bool FAIAssistantConfigTestIgnoresAdditionalFields::RunTest(const FString& UnusedParameters)
{
	const FTemporaryDirectory TemporaryDirectory;
	const FString ConfigFilename(FPaths::Combine(*TemporaryDirectory, FConfig::DefaultFilename));
	verify(
		FFileHelper::SaveStringToFile(
			FString(TEXT(R"json({"ignore_me": "stuff to ignore"})json")),
			*ConfigFilename));
	const FConfig Config = FConfig::Load(ConfigFilename);
	(void)TestEqual(TEXT("main_url"), Config.MainUrl, FConfig::DefaultMainUrl);
	return true;
}

#endif  // WITH_DEV_AUTOMATION_TESTS
