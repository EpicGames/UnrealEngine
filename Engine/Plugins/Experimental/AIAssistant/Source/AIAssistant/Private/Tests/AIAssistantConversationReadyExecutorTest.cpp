// Copyright Epic Games, Inc. All Rights Reserved.

#include "Containers/UnrealString.h"
#include "Misc/AutomationTest.h"

#include "AIAssistantConversationReadyExecutor.h"
#include "AIAssistantExecuteWhenReady.h"
#include "AIAssistantTestFlags.h"

using namespace UE::AIAssistant;

#if WITH_DEV_AUTOMATION_TESTS

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantConversationReadyExecutorSetCreatingConversationTest,
	"AI.Assistant.ConversationReadyExecutor.SetCreatingConversation",
	AIAssistantTest::Flags);

bool FAIAssistantConversationReadyExecutorSetCreatingConversationTest::RunTest(
	const FString& UnusedParameters)
{
	FConversationReadyExecutor ConversationReadyExecutor;
	(void)TestFalse(
		TEXT("No change to conversation state"),
		ConversationReadyExecutor.SetCreatingConversation(false));
	(void)TestFalse(
		TEXT("Creating conversation"),
		ConversationReadyExecutor.SetCreatingConversation(true));
	(void)TestTrue(
		TEXT("Still creating conversation"),
		ConversationReadyExecutor.SetCreatingConversation(true));
	(void)TestTrue(
		TEXT("Created conversation"),
		ConversationReadyExecutor.SetCreatingConversation(false));
	(void)TestFalse(
		TEXT("No change to conversation state after creation"),
		ConversationReadyExecutor.SetCreatingConversation(false));
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantConversationReadyExecutorExecuteOnConstructionTest,
	"AI.Assistant.ConversationReadyExecutor.ExecuteOnConstruction",
	AIAssistantTest::Flags);

bool FAIAssistantConversationReadyExecutorExecuteOnConstructionTest::RunTest(
	const FString& UnusedParameters)
{
	FConversationReadyExecutor ConversationReadyExecutor;
	(void)TestFalse(
		TEXT("NotConfigured"),
		ConversationReadyExecutor.IsAgentEnvironmentConfigured());

	bool bExecuted = false;
	ConversationReadyExecutor.ExecuteWhenReady([&bExecuted]() -> void { bExecuted = true; });
	(void)TestFalse(TEXT("Not configured on construction"), bExecuted);

	ConversationReadyExecutor.NotifyAgentEnvironmentConfigured();
	(void)TestTrue(TEXT("Ready when configured"), bExecuted);
	(void)TestTrue(TEXT("Configured"), ConversationReadyExecutor.IsAgentEnvironmentConfigured());
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantConversationReadyExecutorExecuteAfterResetTest,
	"AI.Assistant.ConversationReadyExecutor.ExecuteAfterReset",
	AIAssistantTest::Flags);

bool FAIAssistantConversationReadyExecutorExecuteAfterResetTest::RunTest(
	const FString& UnusedParameters)
{
	FConversationReadyExecutor ConversationReadyExecutor;
	ConversationReadyExecutor.NotifyAgentEnvironmentConfigured();
	// Block execution.
	(void)ConversationReadyExecutor.SetCreatingConversation(true);

	int ExecutedCount = 0;
	ConversationReadyExecutor.ExecuteWhenReady([&ExecutedCount]() -> void { ++ExecutedCount; });
	// Should not execute anything.
	ConversationReadyExecutor.Reset();
	(void)TestEqual(TEXT("NothingExecuted"), ExecutedCount, 0);

	// Creating a new conversation flushes enqueued executions.
	(void)TestFalse(TEXT("WasReset"), ConversationReadyExecutor.SetCreatingConversation(true));
	ConversationReadyExecutor.ExecuteWhenReady([&ExecutedCount]() -> void { ++ExecutedCount; });
	// Notification should execute the enqueued operation.
	(void)ConversationReadyExecutor.SetCreatingConversation(false);
	ConversationReadyExecutor.NotifyAgentEnvironmentConfigured();
	(void)TestEqual(TEXT("ExecutedOnce"), ExecutedCount, 1);

	// Should execute anything else now we've notified that the conversation is ready.
	ConversationReadyExecutor.ExecuteWhenReady([&ExecutedCount]() -> void { ++ExecutedCount; });
	(void)TestEqual(TEXT("ExecutedTwice"), ExecutedCount, 2);
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantConversationReadyExecutorExecuteAfterConversationCreatedTest,
	"AI.Assistant.ConversationReadyExecutor.ExecuteAfterConversationCreated",
	AIAssistantTest::Flags);

bool FAIAssistantConversationReadyExecutorExecuteAfterConversationCreatedTest::RunTest(
	const FString& UnusedParameters)
{
	FConversationReadyExecutor ConversationReadyExecutor;
	ConversationReadyExecutor.NotifyAgentEnvironmentConfigured();
	(void)ConversationReadyExecutor.SetCreatingConversation(true);
	bool bExecuted = false;
	ConversationReadyExecutor.ExecuteWhenReady([&bExecuted]() -> void { bExecuted = true; });
	(void)TestFalse(TEXT("Not executed while creating conversation"), bExecuted);

	(void)ConversationReadyExecutor.SetCreatingConversation(false);
	(void)TestTrue(TEXT("Executed after conversation created"), bExecuted);
	return true;
}


IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantConversationReadyExecutorOnlyExecuteLastAfterConversationCreatedTest,
	"AI.Assistant.ConversationReadyExecutor.OnlyExecuteLastAfterConversationCreated",
	AIAssistantTest::Flags);

bool FAIAssistantConversationReadyExecutorOnlyExecuteLastAfterConversationCreatedTest::RunTest(
	const FString& UnusedParameters)
{
	FConversationReadyExecutor ConversationReadyExecutor;
	ConversationReadyExecutor.NotifyAgentEnvironmentConfigured();
	(void)ConversationReadyExecutor.SetCreatingConversation(true);
	int ExecutedItem = 0;
	ConversationReadyExecutor.ExecuteWhenReady([&ExecutedItem]() -> void { ExecutedItem = 1; });
	ConversationReadyExecutor.ExecuteWhenReady([&ExecutedItem]() -> void { ExecutedItem = 2; });
	ConversationReadyExecutor.ExecuteWhenReady([&ExecutedItem]() -> void { ExecutedItem = 3; });
	(void)TestEqual(TEXT("Not executed while creating conversation"), ExecutedItem, 0);

	(void)ConversationReadyExecutor.SetCreatingConversation(false);
	(void)TestEqual(TEXT("Executed after conversation created"), ExecutedItem, 3);
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantConversationReadyExecutorDelegateReadyStateTest,
	"AI.Assistant.ConversationReadyExecutor.DelegateReadyState",
	AIAssistantTest::Flags);

bool FAIAssistantConversationReadyExecutorDelegateReadyStateTest::RunTest(
	const FString& UnusedParameters)
{
	FExecuteWhenReady::EExecuteWhenReadyState ExecuteWhenReadyState =
		FExecuteWhenReady::EExecuteWhenReadyState::Wait;
	FConversationReadyExecutor ConversationReadyExecutor(
		[&ExecuteWhenReadyState]() -> FExecuteWhenReady::EExecuteWhenReadyState
		{
			return ExecuteWhenReadyState;
		});
	ConversationReadyExecutor.NotifyAgentEnvironmentConfigured();

	const FString LastExecutionNothing = TEXT("Nothing");
	FString LastExecution = LastExecutionNothing;
	ConversationReadyExecutor.ExecuteWhenReady(
		[&LastExecution]() -> void { LastExecution = TEXT("First"); });
	(void)TestEqual(TEXT("Should wait on the delegate"), LastExecution, LastExecutionNothing);

	ExecuteWhenReadyState = FExecuteWhenReady::EExecuteWhenReadyState::Reject;
	ConversationReadyExecutor.UpdateExecuteWhenReady();
	(void)TestEqual(
		TEXT("Pending execution should be cleared from the queue."),
		LastExecution, LastExecutionNothing);

	ExecuteWhenReadyState = FExecuteWhenReady::EExecuteWhenReadyState::Execute;
	const FString LastExecutionLast = TEXT("Nothing");
	ConversationReadyExecutor.ExecuteWhenReady(
		[&LastExecution, &LastExecutionLast]() -> void
		{
			LastExecution = LastExecutionLast;
		});
	(void)TestEqual(
		TEXT("Only the most recent function should be executed."),
		LastExecution, LastExecutionLast);
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantConversationReadyExecutorNotifyAgentEnvironmentReconfiguringTest,
	"AI.Assistant.ConversationReadyExecutor.NotifyAgentEnvironmentReconfiguring",
	AIAssistantTest::Flags);

bool FAIAssistantConversationReadyExecutorNotifyAgentEnvironmentReconfiguringTest::RunTest(
	const FString& UnusedParameters)
{
	FConversationReadyExecutor ConversationReadyExecutor;
	ConversationReadyExecutor.NotifyAgentEnvironmentConfigured();
	(void)TestTrue(TEXT("Configured"), ConversationReadyExecutor.IsAgentEnvironmentConfigured());

	// reconfiguring should clear the configured flag
	ConversationReadyExecutor.NotifyAgentEnvironmentReconfiguring();
	(void)TestFalse(TEXT("NotConfiguredAfterReconfiguring"), ConversationReadyExecutor.IsAgentEnvironmentConfigured());

	// executor should block when agent is not configured
	bool bExecuted = false;
	ConversationReadyExecutor.ExecuteWhenReady([&bExecuted]() -> void { bExecuted = true; });
	(void)TestFalse(TEXT("NotExecutedWhileReconfiguring"), bExecuted);

	// executor should unblock when agent is configured
	ConversationReadyExecutor.NotifyAgentEnvironmentConfigured();
	(void)TestTrue(TEXT("ExecutedAfterReconfigured"), bExecuted);
	(void)TestTrue(TEXT("ConfiguredAgain"), ConversationReadyExecutor.IsAgentEnvironmentConfigured());
	return true;
}

#endif