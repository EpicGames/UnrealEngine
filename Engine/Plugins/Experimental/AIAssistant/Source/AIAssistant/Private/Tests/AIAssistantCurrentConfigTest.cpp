// Copyright Epic Games, Inc. All Rights Reserved.

#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "Misc/AutomationTest.h"
#include "Misc/CoreDelegates.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Templates/Function.h"

#include "AIAssistantConfig.h"
#include "AIAssistantCurrentConfig.h"
#include "AIAssistantTemporaryDirectory.h"
#include "AIAssistantTestFlags.h"

#if WITH_DEV_AUTOMATION_TESTS

using namespace UE::AIAssistant;

// Creates a fake editor.ini file.
class FEditorIniFile
{
public:
	FEditorIniFile(bool bEnableAssistant) {
		Filename = FPaths::Combine(*Directory, TEXT("Editor.ini"));
		if (!FFileHelper::SaveStringToFile(
			FString::Printf(
				TEXT(
					"[AIAssistant]\n"
					"bIsEnabled = %s\n"),
				bEnableAssistant ? TEXT("true") : TEXT("false")),
			*Filename)) {
			Filename.Empty();
		}
	}

	void With(FAutomationSpecBase& Test, const TFunction<void(const FString&)>& Function) {
		if (Test.TestTrue(TEXT("Created editor ini file"), !Filename.IsEmpty())) {
			Function(Filename);
		}
	}

private:
	FTemporaryDirectory Directory;
	FString Filename;
};

BEGIN_DEFINE_SPEC(AIAssistantCurrentConfig, "AI.Assistant.CurrentConfig",
	AIAssistantTest::Flags)
END_DEFINE_SPEC(AIAssistantCurrentConfig)

void AIAssistantCurrentConfig::Define()
{
	Describe(TEXT("Load"), [this]
		{
			It(TEXT("Should load the configuration"), [this]
				{
					FConfig ExpectedConfig = FConfig::Load();
					FCurrentConfig CurrentConfig;
					auto LoadedConfig = CurrentConfig.Load();
					if (!TestTrue(TEXT("Config is loaded"), LoadedConfig.IsValid())) return;
					(void)TestEqual(TEXT("Config matches expected"),
						LoadedConfig->ToJson(), ExpectedConfig.ToJson());
				});

			It(TEXT("Should reload the configuration"), [this]
				{
					FCurrentConfig CurrentConfig;
					auto LoadedConfig = CurrentConfig.Load();
					if (!TestTrue(TEXT("Config is loaded"), LoadedConfig.IsValid())) return;
					const auto InitialMainUrl = LoadedConfig->MainUrl;
					// Mutate the configuration, this should *not* be performed outside of this test.
					const_cast<FConfig*>(LoadedConfig.Get())->MainUrl = TEXT("https://test-assistant");

					auto ReloadedConfig = CurrentConfig.Load();
					if (!ReloadedConfig) return;
					(void)TestNotEqual(TEXT("Reloaded config should be a different instance"),
						LoadedConfig, ReloadedConfig);
					(void)TestNotEqual(TEXT("Reloaded config should differ"),
						LoadedConfig->ToJson(), ReloadedConfig->ToJson());
					(void)TestEqual(TEXT("Reloaded config should have the original config value"),
						ReloadedConfig->MainUrl, InitialMainUrl);
				});
		});

	Describe(TEXT("GetOrLoad"), [this]
		{
			It(TEXT("Should load the configuration"), [this]
				{
					FConfig ExpectedConfig = FConfig::Load();
					FCurrentConfig CurrentConfig;
					auto LoadedConfig = CurrentConfig.GetOrLoad();
					if (!TestTrue(TEXT("Config is loaded"), LoadedConfig.IsValid())) return;
					(void)TestEqual(TEXT("Config matches expected"),
						LoadedConfig->ToJson(), ExpectedConfig.ToJson());
				});

			It(TEXT("Should return the currently loaded configuration"), [this]
				{
					FCurrentConfig CurrentConfig;
					auto LoadedConfig = CurrentConfig.GetOrLoad();
					if (!TestTrue(TEXT("Config is loaded"), LoadedConfig.IsValid())) return;
					auto Config = CurrentConfig.GetOrLoad();
					(void)TestEqual(TEXT("Config instance should not change"),
						Config, LoadedConfig);
				});
		});

	Describe(TEXT("IsEnabled"), [this]
		{
			It(TEXT("Should return true if enabled in the editor configuration file"), [this]
				{
					FEditorIniFile EditorIniFile(true);
					EditorIniFile.With(
						*this,
						[this](const FString& EditorIniFilename) -> void {
							FCurrentConfig::FConstructorArgs ConstructorArgs;
							ConstructorArgs.EditorIniFilename = EditorIniFilename;
							FCurrentConfig CurrentConfig(ConstructorArgs);
							(void)TestTrue(
								TEXT("Assistant is enabled"), CurrentConfig.IsEnabled());
						});
				});

			It(TEXT("Should return false if disable in the editor configuration file"), [this]
				{
					FEditorIniFile EditorIniFile(false);
					EditorIniFile.With(
						*this,
						[this](const FString& EditorIniFilename) -> void {
							FCurrentConfig::FConstructorArgs ConstructorArgs;
							ConstructorArgs.EditorIniFilename = EditorIniFilename;
							FCurrentConfig CurrentConfig(ConstructorArgs);
							(void)TestFalse(
								TEXT("Assistant is disable"), CurrentConfig.IsEnabled());
						});
				});
		});

	Describe(TEXT("OnPreExit"), [this]
		{
			It(TEXT("Should default to the application OnPreExit delegate"), [this]
				{
					FCurrentConfig CurrentConfig;
					(void)TestEqual(
						TEXT("OnPreExit"),
						&CurrentConfig.OnPreExit(),
						&FCoreDelegates::OnPreExit);
				});
			
			It(TEXT("Should be overridable"), [this]
				{
					FSimpleMulticastDelegate OnTest;
					FCurrentConfig::FConstructorArgs ConstructorArgs;
					ConstructorArgs.OnPreExit = &OnTest;
					FCurrentConfig CurrentConfig(ConstructorArgs);
					(void)TestEqual(TEXT("OnPreExit"), &CurrentConfig.OnPreExit(), &OnTest);
				});
		});

	Describe(TEXT("IsExiting"), [this]
		{
			It(TEXT("Should default to false"), [this]
				{
					FCurrentConfig CurrentConfig;
					(void)TestFalse(TEXT("IsExiting"), CurrentConfig.IsExiting());
				});

			It(TEXT("Should be true after OnPreExit event"), [this]
				{
					FSimpleMulticastDelegate OnTest;
					FCurrentConfig::FConstructorArgs ConstructorArgs;
					ConstructorArgs.OnPreExit = &OnTest;
					FCurrentConfig CurrentConfig(ConstructorArgs);
					OnTest.Broadcast();
					(void)TestTrue(TEXT("IsExiting"), CurrentConfig.IsExiting());
				});
		});
}

#endif