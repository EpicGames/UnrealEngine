// Copyright Epic Games, Inc. All Rights Reserved.

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "Editor.h"
#include "IPythonScriptPlugin.h"
#include "Misc/AutomationTest.h"
#include "PythonScriptTypes.h"
#include "Templates/SharedPointer.h"
#include "Widgets/Docking/SDockTab.h"

#include "AIAssistantCurrentConfig.h"
#include "AIAssistantLog.h"
#include "AIAssistantTextMessage.h"
#include "AIAssistantWebApi.h"
#include "AIAssistantWebApplication.h"
#include "AIAssistantWebBrowser.h"
#include "Tests/AIAssistantTestFlags.h"

#if WITH_DEV_AUTOMATION_TESTS

using namespace UE::AIAssistant;

// These tests create a real browser (SWebBrowser) and a real FWebApplication,
// loading the actual EDA frontend over the network.
//
// They are gated by GIsBuildMachine so they never register on build machines.
// These tests are to be run locally as a way to spot check sending and receiving
// messages and tool calls on the client against a real backend.

// WebApplication subclass that captures agent text responses and tool
// call activity.
class FE2EWebApplication : public FWebApplication
{
public:
	using FWebApplication::FWebApplication;

	// Returns the last agent text response, or empty if none received.
	FString GetLastAgentResponse() const { return LastAgentResponse; }

	// Whether any agent text response has been received.
	bool HasAgentResponse() const { return !LastAgentResponse.IsEmpty(); }

	// Whether any tool call was processed.
	bool HasProcessedToolCall() const { return ProcessedToolCallNames.Num() > 0; }

	// Returns the names of all tool calls that were processed.
	const TArray<FString>& GetProcessedToolCallNames() const { return ProcessedToolCallNames; }

protected:
	bool ProcessMessage(const FMessage& Message, const FString& ConversationId) override
	{
		bool bProcessedMessage = FWebApplication::ProcessMessage(Message, ConversationId);

		for (const FMessageContent& ContentItem : Message.MessageContent)
		{
			if (ContentItem.ContentType == EMessageContentType::ToolCall)
			{
				const FToolCallContent& ToolCall =
					ContentItem.Content.Get<FToolCallContent>();
				ProcessedToolCallNames.Add(ToolCall.Name);
			}
			if (Message.MessageRole == EMessageRole::Agent
				&& ContentItem.ContentType == EMessageContentType::Text)
			{
				LastAgentResponse =
					ContentItem.Content.Get<FTextMessageContent>().Text;
			}
		}
		return bProcessedMessage;
	}

private:
	FString LastAgentResponse;
	TArray<FString> ProcessedToolCallNames;
};

// Extends SAIAssistantWebBrowser to create FE2EWebApplication instead of the
// default FWebApplication.
class SE2EWebBrowser : public SAIAssistantWebBrowser
{
public:
	SLATE_BEGIN_ARGS(SE2EWebBrowser) {}
		SLATE_ARGUMENT(TSharedPtr<FCurrentConfig>, CurrentConfig);
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<SDockTab>& InParentTab)
	{
		SAIAssistantWebBrowser::Construct(
			SAIAssistantWebBrowser::FArguments()
				.CurrentConfig(InArgs._CurrentConfig),
			InParentTab);
	}

	TSharedPtr<FE2EWebApplication> GetE2EWebApplication() const
	{
		return E2EWebApplication;
	}

protected:
	TSharedPtr<FWebApplication> CreateWebApplication(
		IWebJavaScriptExecutor& Executor,
		IWebJavaScriptDelegateBinder& Binder,
		FSimpleMulticastDelegate& OnPreExit) override
	{
		auto Factory = FWebApplication::CreateWebApiFactory(
			Executor, Binder, OnPreExit);
		E2EWebApplication = MakeShared<FE2EWebApplication>(MoveTemp(Factory));
		return E2EWebApplication;
	}

private:
	TSharedPtr<FE2EWebApplication> E2EWebApplication;
};

// Holds the SE2EWebBrowser widget, WebApplication, and config for E2E tests.
// Member declaration order matters: destruction happens in reverse order. OnPreExit
// must outlive Browser because FCurrentConfig holds a reference to it.
struct FE2ETestEnvironment : public TSharedFromThis<FE2ETestEnvironment>
{
	FAutomationTestBase& Spec;
	FSimpleMulticastDelegate OnPreExit;
	TSharedPtr<FE2EWebApplication> WebApplication;
	TSharedPtr<SE2EWebBrowser> Browser;

	explicit FE2ETestEnvironment(FAutomationTestBase& InSpec) : Spec(InSpec) {}

	bool Initialize()
	{
		FCurrentConfig::FConstructorArgs ConfigArgs;
		ConfigArgs.OnPreExit = &OnPreExit;
		TSharedPtr<FCurrentConfig> Config = MakeShared<FCurrentConfig>(ConfigArgs);

		// Slate requires a parent dock tab for proper widget nesting.
		// Provide a dummy tab since there is no real editor tab in tests.
		TSharedRef<SDockTab> DummyDockTab = SNew(SDockTab);

		Browser = SNew(SE2EWebBrowser, DummyDockTab).CurrentConfig(Config);
		if (!Browser.IsValid())
		{
			return false;
		}

		WebApplication = Browser->GetE2EWebApplication();
		return WebApplication.IsValid();
	}

	// Queues latent commands that wait for the WebApplication to reach Complete state.
	void QueueWaitForReady()
	{
		TSharedPtr<FE2ETestEnvironment> Self = AsShared();
		Spec.AddCommand(new FUntilCommand(
			[Self]()
			{
				return Self->WebApplication->GetLoadState() ==
					FWebApplication::ELoadState::Complete;
			},
			[Self]()
			{
				Self->Spec.AddError(
					TEXT("E2E: WebApplication did not reach Complete state"));
				return true;
			},
			60.0f));
	}

	// Queues latent commands that send a user message, wait for the agent to respond,
	// log the response, and verify the app remains healthy.
	void QueueSendMessageAndWaitForResponse(
		const FString& MessageText, double ResponseTimeoutSeconds)
	{
		TSharedPtr<FE2ETestEnvironment> Self = AsShared();

		// Send the user message.
		Spec.AddCommand(new FDelayedFunctionLatentCommand(
			[Self, MessageText]()
			{
				Self->WebApplication->AddUserMessageToConversation(
					CreateUserMessage(MessageText, TEXT("")));
			},
			0.0f));

		// Wait for the agent to respond.
		Spec.AddCommand(new FUntilCommand(
			[Self]()
			{
				return Self->WebApplication->HasAgentResponse();
			},
			[Self]()
			{
				Self->Spec.AddError(TEXT("E2E: Timed out waiting for agent response"));
				return true;
			},
			static_cast<float>(ResponseTimeoutSeconds)));

		// Log the response and verify the WebApplication is still healthy.
		Spec.AddCommand(new FDelayedFunctionLatentCommand(
			[Self]()
			{
				FString Response =
					Self->WebApplication->GetLastAgentResponse();
				UE_LOGF(LogAIAssistant, Display,
					"E2E Agent Response: %ls", *Response);
				Self->Spec.AddInfo(FString::Printf(
					TEXT("Agent Response: %s"), *Response));

				(void)Self->Spec.TestFalse(
					TEXT("E2E_ResponseNotEmpty"),
					Response.IsEmpty());

				(void)Self->Spec.TestNotEqual(
					TEXT("E2E_StillHealthy"),
					Self->WebApplication->GetLoadState(),
					FWebApplication::ELoadState::Error);
			},
			0.0f));
	}

	// Register the DemoPythonToolset via the Python ToolsetRegistry API. Returns false
	// if Python is not available or registration fails.
	bool RegisterDemoPythonToolset()
	{
		if (!IPythonScriptPlugin::Get()->IsPythonAvailable())
		{
			return false;
		}

		FPythonCommandEx PythonCommand;
		PythonCommand.ExecutionMode = EPythonCommandExecutionMode::EvaluateStatement;
		PythonCommand.Command = TEXT(
			"exec('from toolset_registry.tests.demo_toolset import DemoPythonToolset') "
			"or unreal.ToolsetRegistry.register_toolset_class(DemoPythonToolset)");
		return IPythonScriptPlugin::Get()->ExecPythonCommandEx(PythonCommand);
	}

	// Unregister the DemoPythonToolset via the Python ToolsetRegistry API.
	void UnregisterDemoPythonToolset()
	{
		if (!IPythonScriptPlugin::Get()->IsPythonAvailable())
		{
			return;
		}

		FPythonCommandEx PythonCommand;
		PythonCommand.ExecutionMode = EPythonCommandExecutionMode::EvaluateStatement;
		PythonCommand.Command = TEXT(
			"exec('from toolset_registry.tests.demo_toolset import DemoPythonToolset') "
			"or unreal.ToolsetRegistry.unregister_toolset_class(DemoPythonToolset)");
		IPythonScriptPlugin::Get()->ExecPythonCommandEx(PythonCommand);
	}

	~FE2ETestEnvironment()
	{
		OnPreExit.Broadcast();
		Browser.Reset();
		WebApplication.Reset();
		UnregisterDemoPythonToolset();
	}
};

BEGIN_DEFINE_SPEC(
	FAIAssistantEndToEndTest, "AI.Assistant.EndToEnd",
	AIAssistantTest::Flags)
END_DEFINE_SPEC(FAIAssistantEndToEndTest)

void FAIAssistantEndToEndTest::Define()
{
	// Never register these tests on build machines.
	if (GIsBuildMachine)
	{
		return;
	}

	Describe(TEXT("SendMessage"), [this]
	{
		It(TEXT("should send a simple message without error"), [this]
		{
			TSharedPtr<FE2ETestEnvironment> Env =
				MakeShared<FE2ETestEnvironment>(*this);
			if (!TestTrue(TEXT("E2E_Initialize"), Env->Initialize()))
			{
				return;
			}
			Env->QueueWaitForReady();
			Env->QueueSendMessageAndWaitForResponse(TEXT("hi"), 30.0);
		});
	});

	Describe(TEXT("ToolCall"), [this]
	{
		It(TEXT("should trigger a tool call and remain healthy"), [this]
		{
			TSharedPtr<FE2ETestEnvironment> Env =
				MakeShared<FE2ETestEnvironment>(*this);
			if (!TestTrue(TEXT("E2E_Initialize"), Env->Initialize()))
			{
				return;
			}
			if (!TestTrue(TEXT("E2E_RegisterToolset"),
				Env->RegisterDemoPythonToolset()))
			{
				return;
			}

			Env->QueueWaitForReady();
			Env->QueueSendMessageAndWaitForResponse(
				TEXT("Please call the hello_world tool with "
						"msg set to 'E2E test'"),
				60.0);

			// Verify the hello_world tool was actually invoked.
			AddCommand(new FDelayedFunctionLatentCommand(
				[this, Env]()
				{
					(void)TestTrue(
						TEXT("E2E_ToolCallProcessed"),
						Env->WebApplication->HasProcessedToolCall());

					const TArray<FString>& ToolCallNames =
						Env->WebApplication->GetProcessedToolCallNames();
					(void)TestTrue(
						TEXT("E2E_HelloWorldToolCalled"),
						ToolCallNames.FindLastByPredicate(
							[](const FString& ToolCall) {
								return ToolCall.Contains(TEXT("hello_world"));
							}) != INDEX_NONE);

					// Verify the tool output appeared in the agent
					// response. DemoPythonToolset.hello_world returns
					// "Hello, world! <msg> from demo toolset".
					FString Response =
						Env->WebApplication->GetLastAgentResponse();
					(void)TestTrue(
						TEXT("E2E_ResponseContainsToolOutput"),
						Response.Contains(TEXT("Hello, world!")));
				},
				0.0f));
		});
	});
}

#endif
