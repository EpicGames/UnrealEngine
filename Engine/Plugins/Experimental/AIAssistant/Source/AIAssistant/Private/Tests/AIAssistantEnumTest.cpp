// Copyright Epic Games, Inc. All Rights Reserved.

#include <cstddef>
#include "Containers/UnrealString.h"
#include "Misc/AutomationTest.h"

#include "AIAssistantEnum.h"
#include "AIAssistantTestFlags.h"

#if WITH_DEV_AUTOMATION_TESTS

using UE::AIAssistant::GetEnumValueFromDescription;
using UE::AIAssistant::GetEnumValueDescription;

#define ADJECTIVE_ENUM(X)         \
	X(EAdjective::Bish, "Whack"), \
	X(EAdjective::Bosh, "Kapow")

enum class EAdjective
{
	Bish = 0,
	Bosh = 1
};

static const EAdjective Adjectives[] = { ADJECTIVE_ENUM(UE_ENUM_VALUE) };

static constexpr uint32 AdjectivesCount =
  UE_ENUM_COUNT(ADJECTIVE_ENUM(UE_ENUM_COUNTER));

static const UE_ENUM_VALUE_DESCRIPTION_TYPE(EAdjective, ADJECTIVE_ENUM)
	AdjectiveDescriptions =
{
	ADJECTIVE_ENUM(UE_ENUM_VALUE_DESCRIPTION)
};

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantEnumTestCount,
	"AI.Assistant.Enum.Count",
	AIAssistantTest::Flags);

// Ensure the enum value count is correct.
bool FAIAssistantEnumTestCount::RunTest(const FString& UnusedParameters)
{
	return TestEqual(TEXT("EnumCount"), static_cast<int>(AdjectivesCount), 2);
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantEnumTestQueryValues,
	"AI.Assistant.Enum.QueryValues",
	AIAssistantTest::Flags);

// Query the static enum value array.
bool FAIAssistantEnumTestQueryValues::RunTest(const FString& UnusedParameters)
{
	(void)TestEqual(TEXT("Adjectives[0]"), Adjectives[0], EAdjective::Bish);
	(void)TestEqual(TEXT("Adjectives[1]"), Adjectives[1], EAdjective::Bosh);
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantEnumTestGetEnumValueDescription,
	"AI.Assistant.Enum.GetEnumValueDescription",
	AIAssistantTest::Flags);

// Query enum descriptions.
bool FAIAssistantEnumTestGetEnumValueDescription::RunTest(const FString& UnusedParameters)
{
	(void)TestEqual(
		TEXT("EAdjective::Bish"),
		**GetEnumValueDescription(AdjectiveDescriptions, EAdjective::Bish),
		TEXT("Whack"));
	(void)TestEqual(
		TEXT("EAdjective::Bosh"),
		**GetEnumValueDescription(AdjectiveDescriptions, EAdjective::Bosh),
		TEXT("Kapow"));
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantEnumTestGetEnumValueDescriptionOutOfRange,
	"AI.Assistant.Enum.GetEnumValueDescriptionOutOfRange",
	AIAssistantTest::Flags);

// Ensure no value is returned for out of range enum queries.
bool FAIAssistantEnumTestGetEnumValueDescriptionOutOfRange::RunTest(const FString& UnusedParameters)
{
	(void)TestFalse(
		TEXT("-1"),
		GetEnumValueDescription(AdjectiveDescriptions, static_cast<EAdjective>(-1)).IsSet());
	(void)TestFalse(
		TEXT("99"),
		GetEnumValueDescription(AdjectiveDescriptions, static_cast<EAdjective>(99)).IsSet());
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantEnumTestGetEnumValueFromDescription,
	"AI.Assistant.Enum.GetEnumValueFromDescription",
	AIAssistantTest::Flags);

// Convert a description to an enum value.
bool FAIAssistantEnumTestGetEnumValueFromDescription::RunTest(const FString& UnusedParameters)
{
	(void)TestEqual(
		TEXT("EAdjective::Bish"),
		GetEnumValueFromDescription(AdjectiveDescriptions, TEXT("Whack")),
		TOptional<EAdjective>(EAdjective::Bish));
	(void)TestEqual(
		TEXT("EAdjective::Bosh"),
		GetEnumValueFromDescription(AdjectiveDescriptions, TEXT("Kapow")),
		TOptional<EAdjective>(EAdjective::Bosh));
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantEnumTestGetEnumValueFromDescriptionInvalid,
	"AI.Assistant.Enum.GetEnumValueFromDescriptionInvalid",
	AIAssistantTest::Flags);

// Ensure no value is returned for an invalid enum description.
bool FAIAssistantEnumTestGetEnumValueFromDescriptionInvalid::RunTest(const FString& UnusedParameters)
{
	(void)TestFalse(
		TEXT("No value"),
		GetEnumValueFromDescription(AdjectiveDescriptions, TEXT("Invalid")).IsSet());
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantEnumTestGetEnumValueFromDescriptionIgnoringCase,
	"AI.Assistant.Enum.GetEnumValueFromDescriptionIgnoringCase",
	AIAssistantTest::Flags);

// Convert a description, ignoring case, to an enum value.
bool FAIAssistantEnumTestGetEnumValueFromDescriptionIgnoringCase::RunTest(const FString& UnusedParameters)
{
	(void)TestEqual(
		TEXT("Adjective::Bish"),
		*GetEnumValueFromDescription(AdjectiveDescriptions, TEXT("wHaCk"), ESearchCase::IgnoreCase),
		EAdjective::Bish);
	(void)TestEqual(
		TEXT("Adjective::Bosh"),
		*GetEnumValueFromDescription(AdjectiveDescriptions, TEXT("kaPOW"), ESearchCase::IgnoreCase),
		EAdjective::Bosh);
	return true;
}

UE_ENUM_METADATA_DECLARE(EAdjective, ADJECTIVE_ENUM);
UE_ENUM_METADATA_DEFINE(EAdjective, ADJECTIVE_ENUM);

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantEnumTestMetadataMacro,
	"AI.Assistant.Enum.MetadataMacro",
	AIAssistantTest::Flags);

// Test UE_ENUM_METADATA() macro.
bool FAIAssistantEnumTestMetadataMacro::RunTest(const FString& UnusedParameters)
{
	(void)TestEqual(
		TEXT("AdjectivesCount"),
		EAdjectiveCount,
		AdjectivesCount);
	(void)TestEqual(
		TEXT("Adjective::Bish"),
		LexToString(EAdjective::Bish),
		**GetEnumValueDescription(EAdjectiveDescriptions, EAdjective::Bish));
	
	EAdjective AdjectiveFromString;
	LexFromString(AdjectiveFromString, TEXT("Kapow"));
	(void)TestEqual(
		TEXT("Adjective::Bosh"),
		AdjectiveFromString,
		EAdjective::Bosh);
	return true;
}

#endif  // WITH_DEV_AUTOMATION_TESTS
