// Copyright Epic Games, Inc. All Rights Reserved.
	

#include "AIAssistantFakeExecuteWhenReady.h"
#include "AIAssistantTestFlags.h"


#if WITH_DEV_AUTOMATION_TESTS


using namespace UE::AIAssistant;


void GeneralFakeExecuteWhenReadyTestFunction(FAutomationTestBase& AutomationTest, const FFakeExecuteWhenReady::ETestMode TestMode)
{
	static const FString TestStringWhenWaiting = TEXT("Waiting");
	static const FString TestStringWhenExecuted = TEXT("Executed");
	
	FString TestStringCurrent = TestStringWhenWaiting;

	
	FFakeExecuteWhenReady FakeExecuteWhenReady(TestMode);
	
	FakeExecuteWhenReady.ExecuteWhenReady([&TestStringCurrent]()
		{
			TestStringCurrent = TestStringWhenExecuted;
		});

	FakeExecuteWhenReady.ExecuteWhenReady([]
		{
			// Empty function added, to test function count.
		});

	
	for (int32 I = 0; I <= FakeExecuteWhenReady.FakeStateTransitionCount + 10; I++)
	{
		FakeExecuteWhenReady.SetFakeStateCount(I);
		
		FakeExecuteWhenReady.UpdateExecuteWhenReady();

		if (TestMode == FFakeExecuteWhenReady::ExecuteWhenStateHitsValue)
		{
			if (I >= FFakeExecuteWhenReady::FakeStateTransitionCount)
			{
				AutomationTest.TestEqual("TestString", TestStringCurrent, TestStringWhenExecuted);
				AutomationTest.TestEqual("DeferredExecutionFunctionCount", FakeExecuteWhenReady.GetNumDeferredExecutionFunctions(), 0);
			}
			else 
			{
				AutomationTest.TestEqual("TestString", TestStringCurrent, TestStringWhenWaiting);
				AutomationTest.TestEqual("DeferredExecutionFunctionCount", FakeExecuteWhenReady.GetNumDeferredExecutionFunctions(), 2);
			}
		}
		else if (TestMode == FFakeExecuteWhenReady::RejectWhenStateHitsValue)
		{
			if (I >= FFakeExecuteWhenReady::FakeStateTransitionCount)
			{
				AutomationTest.TestEqual("TestString", TestStringCurrent, TestStringWhenWaiting);
				AutomationTest.TestEqual("DeferredExecutionFunctionCount", FakeExecuteWhenReady.GetNumDeferredExecutionFunctions(), 0);
			}
			else 
			{
				AutomationTest.TestEqual("TestString", TestStringCurrent, TestStringWhenWaiting);
				AutomationTest.TestEqual("DeferredExecutionFunctionCount", FakeExecuteWhenReady.GetNumDeferredExecutionFunctions(), 2);
			}
		}
		else if (TestMode == FFakeExecuteWhenReady::IgnoreWhenStateHitsValue)
		{
			AutomationTest.TestEqual("TestString", TestStringCurrent, TestStringWhenWaiting);
			AutomationTest.TestEqual("DeferredExecutionFunctionCount", FakeExecuteWhenReady.GetNumDeferredExecutionFunctions(), 2);
		}
	}
}


// Test what happens when ExecuteWhenReady actually executes when the time comes.

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	AIAssistantExecuteWhenReadyTestExecute,
	"AI.Assistant.ExecuteWhenReady.Execute",
	AIAssistantTest::Flags);

bool AIAssistantExecuteWhenReadyTestExecute::RunTest(const FString& UnusedParameters)
{
	GeneralFakeExecuteWhenReadyTestFunction(*this, FFakeExecuteWhenReady::ExecuteWhenStateHitsValue);
	
	return true;
}


// Test what happens when ExecuteWhenReady rejects execution when the time comes.

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	AIAssistantExecuteWhenReadyTestReject,
	"AI.Assistant.ExecuteWhenReady.Reject",
	AIAssistantTest::Flags);

bool AIAssistantExecuteWhenReadyTestReject::RunTest(const FString& UnusedParameters)
{
	GeneralFakeExecuteWhenReadyTestFunction(*this, FFakeExecuteWhenReady::RejectWhenStateHitsValue);
	
	return true;
}


// Test what happens when ExecuteWhenReady ignores execution when the time comes.

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	AIAssistantExecuteWhenReadyTestIgnore,
	"AI.Assistant.ExecuteWhenReady.Ignore",
	AIAssistantTest::Flags);

bool AIAssistantExecuteWhenReadyTestIgnore::RunTest(const FString& UnusedParameters)
{
	GeneralFakeExecuteWhenReadyTestFunction(*this, FFakeExecuteWhenReady::IgnoreWhenStateHitsValue);
	
	return true;
}


#endif
