// Copyright Epic Games, Inc. All Rights Reserved.
	

#include "AIAssistantFakeExecuteWhenReady.h"

#include "AIAssistantExecuteWhenReady.h"


UE::AIAssistant::FFakeExecuteWhenReady::FFakeExecuteWhenReady(ETestMode InTestMode) :
	TestMode(InTestMode)
{
}


void UE::AIAssistant::FFakeExecuteWhenReady::SetFakeStateCount(const int32 InFakeStateCount)
{
	FakeStateCount = InFakeStateCount;
}


UE::AIAssistant::FExecuteWhenReady::EExecuteWhenReadyState UE::AIAssistant::FFakeExecuteWhenReady::GetExecuteWhenReadyState()
{
	if (TestMode == ExecuteWhenStateHitsValue)
	{
		return (FakeStateCount == FakeStateTransitionCount ? EExecuteWhenReadyState::Execute : EExecuteWhenReadyState::Wait);
	}
	else if (TestMode == RejectWhenStateHitsValue)
	{
		return (FakeStateCount == FakeStateTransitionCount ? EExecuteWhenReadyState::Reject : EExecuteWhenReadyState::Wait);
	}
	else 
	{
		return EExecuteWhenReadyState::Wait;
	}
}
