// Copyright Epic Games, Inc. All Rights Reserved.
	

#pragma once


#include "AIAssistantExecuteWhenReady.h"


namespace UE::AIAssistant
{
	class FFakeExecuteWhenReady : public FExecuteWhenReady
	{
	public:

		enum ETestMode
		{
			ExecuteWhenStateHitsValue,
			RejectWhenStateHitsValue,
			IgnoreWhenStateHitsValue,
		};

		
		explicit FFakeExecuteWhenReady(ETestMode InTestMode);

		
		static constexpr int32 FakeStateTransitionCount = 10;

		void SetFakeStateCount(const int32 InFakeStateCount);

		
		virtual EExecuteWhenReadyState GetExecuteWhenReadyState() override;

		
	private:


		ETestMode TestMode = ExecuteWhenStateHitsValue;

		int32 FakeStateCount = 0;
	};
}
