// Copyright Epic Games, Inc. All Rights Reserved.

#include "AIAssistantFakeToolset.h"

#include "Templates/UnrealTemplate.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::AIAssistant
{
	FFakeToolset::FFakeToolset(const FString& Name) :
		Name(Name)
	{
	}

	FFakeToolset::~FFakeToolset() = default;

	TFuture<TValueOrError<FString, FString>> FFakeToolset::ExecuteToolInternal(
		const FString& ToolName, const FString& JsonInput)
	{
		TSharedPtr<TPromise<TValueOrError<FString, FString>>> Promise;
		if (!ToolPromises.RemoveAndCopyValue(ToolName, Promise) || !Promise)
		{
			Promise =
				MakeShared<TPromise<TValueOrError<FString, FString>>>(
					MakeFulfilledPromise<TValueOrError<FString, FString>>(
						MakeError(FString(TEXT("Tool not found")))));
		}
		return Promise->GetFuture().Next(
			[this, ToolName = ToolName](const TValueOrError<FString, FString>& Result) ->
				TValueOrError<FString, FString>
			{
				ToolResultsByName.FindOrAdd(ToolName).Add(Result);
				return Result;
			});
	}

	FString FFakeToolset::GetJsonSchemaInternal() const
	{
		// This does not generate actual JSON schema. This is just a simplified form
		// for testing.
		FString Tools;
		bool bFirst = true;
		for (const auto& [ToolName, ToolPromise] : ToolPromises)
		{
			if (!bFirst) Tools += ",";
			bFirst = false;
			Tools += FString::Printf(TEXT(R"json("%s.%s")json"), *GetToolsetName(), *ToolName);
		}
		FString ToolsetSchema = FString::Printf(TEXT(
			R"json({"name":"%s","tools":[%s]})json"), *GetToolsetName(), *Tools);
		return ToolsetSchema;
	}

	FString FFakeToolset::GetToolsetName() const
	{
		return Name;
	}

	FString FFakeToolset::GetToolsetVersion() const
	{
		return TEXT("1.0");
	}

	FString FFakeToolset::GetToolsetDescription() const
	{
		return TEXT("Fake Toolset for testing.");
	}

	TSharedPtr<TPromise<TValueOrError<FString, FString>>> FFakeToolset::AddFakeToolCall(
		const FString& ToolName,
		TPromise<TValueOrError<FString, FString>>&& Promise)
	{
		return ToolPromises.Add(
			ToolName, MakeShared<TPromise<TValueOrError<FString, FString>>>(MoveTemp(Promise)));
	}
}

#endif