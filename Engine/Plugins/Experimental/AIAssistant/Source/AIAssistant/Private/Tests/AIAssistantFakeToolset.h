// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Async/Future.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Misc/AutomationTest.h"
#include "Templates/SharedPointer.h"
#include "Templates/ValueOrError.h"
#include "UObject/StrongObjectPtr.h"

#include "ToolsetRegistry/Toolset.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::AIAssistant
{

class FFakeToolset : public UE::ToolsetRegistry::FToolset
{
public:
	static const inline FString SUCCESSFUL_RESULT = FString(TEXT(R"({"result": "success"})"));

	FFakeToolset(const FString& Name);
	virtual ~FFakeToolset();

	TFuture<TValueOrError<FString, FString>> ExecuteToolInternal(
		const FString& ToolName, const FString& JsonInput) override;

	FString GetJsonSchemaInternal() const override;

	FString GetToolsetName() const override;

	FString GetToolsetVersion() const override;
	FString GetToolsetDescription() const override;

	TSharedPtr<TPromise<TValueOrError<FString, FString>>> AddFakeToolCall(
		const FString& ToolName,
		TPromise<TValueOrError<FString, FString>>&& Promise =
			TPromise<TValueOrError<FString, FString>>());

public:
	// Tool results by tool name.
	TMap<FString, TArray<TValueOrError<FString, FString>>> ToolResultsByName;

private:
	FString Name;
	TMap<FString, TSharedPtr<TPromise<TValueOrError<FString, FString>>>> ToolPromises;
};

}

#endif
