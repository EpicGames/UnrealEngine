// Copyright Epic Games, Inc. All Rights Reserved.

#include "AIAssistantFakeWebApi.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::AIAssistant
{
	TUniquePtr<FFakeWebApi::FExecutedAsyncFunction> FFakeWebApi::TestExpectAsyncFunctionCall(
		FAutomationTestBase& TestCase,
		const TCHAR* InstanceName,
		const TCHAR* FunctionName,
		const TCHAR* Arguments) const
	{
		TUniquePtr<FExecutedAsyncFunction> ReturnFunction;
		auto FoundFunctions = FindExecutedAsyncFunctions(InstanceName, FunctionName);
		if (TestCase.TestEqual(FunctionName, 1, FoundFunctions.Num()))
		{
			const auto& FoundFunction = FoundFunctions[0];
			bool bCalled;
			bCalled = TestCase.TestEqual(
				FString::Printf(TEXT("%s_Called"), FunctionName),
				FoundFunction.FunctionName, FunctionName);
			bCalled = TestCase.TestEqual(
				FString::Printf(TEXT("%s_Args"), FunctionName),
				FoundFunction.Arguments, Arguments) && bCalled;
			ReturnFunction = MakeUnique<FExecutedAsyncFunction>(FoundFunction);
		}
		return ReturnFunction;
	}

	bool FFakeWebApi::TestExpectAsyncFunctionCallAndComplete(
		FAutomationTestBase& TestCase,
		const TCHAR* InstanceName,
		const TCHAR* FunctionName,
		const TCHAR* Arguments,
		const TCHAR* ResultJson,
		bool bResultJsonIsError)
	{
		auto FoundFunction =
			TestExpectAsyncFunctionCall(TestCase, InstanceName, FunctionName, Arguments);
		if (!FoundFunction.IsValid()) return false;

		CompleteAsyncFunction(*FoundFunction, ResultJson, bResultJsonIsError);
		return true;
	}

	TArray<FFakeWebApi::FExecutedAsyncFunction> FFakeWebApi::FindExecutedAsyncFunctions(
		const TCHAR* InstanceName, const FString& FunctionName) const
	{
		TArray<FExecutedAsyncFunction> FoundFunctions;
		const TCHAR* ExpectedInstanceName =
			InstanceName ? InstanceName : *FWebApi::WebApiObjectName;
		for (const auto& Function : ExecutedAsyncFunctions)
		{
			if (Function.InstanceName == ExpectedInstanceName &&
				Function.FunctionName == FunctionName)
			{
				FoundFunctions.Add(Function);
			}
		}
		return FoundFunctions;
	}

	void FFakeWebApi::RemoveAsyncFunctions(const FString& FunctionName)
	{
		ExecutedAsyncFunctions.RemoveAll(
			[&FunctionName](FExecutedAsyncFunction& Function)
			{
				return Function.FunctionName == FunctionName;
			});
	}

	void FFakeWebApi::CompleteAsyncFunction(
		const FExecutedAsyncFunction& ExecutedAsyncFunction,
		const FString& ResultJson, bool bResultJsonIsError)
	{
		FWebJavaScriptResultDelegateAccessor::CallHandleResult(
			*WebJavaScriptResultDelegate, ExecutedAsyncFunction.HandlerId,
			ResultJson, bResultJsonIsError);
		// Remove the completed entry so it no longer appears in FindExecutedAsyncFunctions.
		// This ensures repeated calls to TestExpectAsyncFunctionCallAndComplete on the same
		// function name (e.g. across multiple page loads) each find exactly one match.
		ExecutedAsyncFunctions.RemoveAll(
			[&ExecutedAsyncFunction](const FExecutedAsyncFunction& Other)
			{
				return Other.HandlerId == ExecutedAsyncFunction.HandlerId;
			});
	}

	void FFakeWebApi::InvokeCallback(
		const FString& HandlerId, const FString& ResultJson, bool bResultJsonIsError)
	{
		FWebJavaScriptResultDelegateAccessor::CallHandleResult(
			*WebJavaScriptResultDelegate, HandlerId, ResultJson, bResultJsonIsError);
	}

	FString FFakeWebApi::RegisterCallback(
		TFunction<UAIAssistantWebJavaScriptResultDelegate::EHandlerStatus(
			UAIAssistantWebJavaScriptResultDelegate::FResult)> Callback)
	{
		RegisteredCallbackHandlerIds.Add(FWebApi::RegisterCallback(MoveTemp(Callback)));
		return RegisteredCallbackHandlerIds.Last();
	}

	void FFakeWebApi::ExecuteAsyncFunction(
		const TCHAR* InstanceName, const TCHAR* FunctionName, const TCHAR* Arguments,
		const TCHAR* HandlerId)
	{
		FWebApi::ExecuteAsyncFunction(InstanceName, FunctionName, Arguments, HandlerId);
		ExecutedAsyncFunctions.Emplace(
			FExecutedAsyncFunction{
				InstanceName ? InstanceName : TEXT(""),
				FunctionName, Arguments, HandlerId
			});
	}
}

#endif