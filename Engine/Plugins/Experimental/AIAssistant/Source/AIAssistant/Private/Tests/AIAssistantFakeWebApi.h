// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Misc/AssertionMacros.h"
#include "Misc/AutomationTest.h"
#include "Templates/UniquePtr.h"
#include "Templates/ValueOrError.h"

#include "AIAssistantWebApi.h"
#include "Tests/AIAssistantWebJavaScriptResultDelegateAccessor.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::AIAssistant
{
	class FFakeWebApi : public FWebApi
	{
	public:
		// Function executed by the FWebJavaScriptFunctionExecutor used to construct FFakeWebApi.
		struct FExecutedAsyncFunction
		{
			// Instance used to execute the function or an empty string for no instance.
			FString InstanceName;
			// Name of the executed function.
			FString FunctionName;
			// Arguments passed to the function.
			FString Arguments;
			// Handler for the result of the function.
			FString HandlerId;
		};

	public:
		using FWebApi::FWebApi;
		virtual ~FFakeWebApi() = default;

		// Ensure a JavaScript function was called with the specified arguments.
		// If InstanceName is null, the default object instance WebApiObjectName is expected.
		TUniquePtr<FExecutedAsyncFunction> TestExpectAsyncFunctionCall(
			FAutomationTestBase& TestCase,
			const TCHAR* InstanceName,
			const TCHAR* FunctionName,
			const TCHAR* Arguments = TEXT("")) const;

		// Ensure a JavaScript function was called with the specified arguments and complete it
		// with ResultJson as a result of error if bResultJsonIsError is true.
		// If InstanceName is null, the default object instance WebApiObjectName is expected.
		bool TestExpectAsyncFunctionCallAndComplete(
			FAutomationTestBase& TestCase,
			const TCHAR* InstanceName,
			const TCHAR* FunctionName,
			const TCHAR* Arguments,
			const TCHAR* ResultJson,
			bool bResultJsonIsError);

		// Ensure a JavaScript function was called with the specified arguments and complete it
		// with ResultJson as a result of error if bResultJsonIsError is true. Also, validate the
		// future receives the appropriate TValueOrError from ResultJson and bResultJsonIsError.
		// If InstanceName is null, the default object instance WebApiObjectName is expected.
		template<typename ResultFutureType>
		bool TestExpectAsyncFunctionCallAndComplete(
			FAutomationTestBase& TestCase,
			const TCHAR* InstanceName,
			const TCHAR* FunctionName,
			const TCHAR* Arguments,
			ResultFutureType& ResultFuture,
			const TCHAR* ResultJson,
			bool bResultJsonIsError)
		{
			check(FunctionName);
			if (!(TestCase.TestFalse(
					FString::Printf(TEXT("%s_CompleteBeforeHandler"), FunctionName),
					ResultFuture.IsReady()) &&
				TestExpectAsyncFunctionCallAndComplete(
					TestCase, InstanceName, FunctionName, Arguments, ResultJson,
					bResultJsonIsError) &&
				TestCase.TestTrue(
					FString::Printf(TEXT("%s_CompleteAfterHandler"), FunctionName),
					ResultFuture.IsReady())))
			{
				return false;
			}
			const auto& Result = ResultFuture.Get();
			bool bCompletedAsExpected;
			bCompletedAsExpected = TestCase.TestEqual(
				FString::Printf(TEXT("%s_ResultHasError"), FunctionName),
				Result.HasError(), bResultJsonIsError);
			bCompletedAsExpected = TestCase.TestEqual(
				FString::Printf(TEXT("%s_HasExpectedResult"), FunctionName),
				bResultJsonIsError ? Result.GetError() : GetValueOrEmptyString(Result),
				ResultJson) && bCompletedAsExpected;
			return bCompletedAsExpected;
		}

		// Find executed functions by name.
		// If InstanceName is null, the default object instance WebApiObjectName is used in the 
		// search.
		TArray<FExecutedAsyncFunction> FindExecutedAsyncFunctions(
			const TCHAR* InstanceName, const FString& FunctionName) const;

		// Remove executed functions by name.
		void RemoveAsyncFunctions(const FString& FunctionName);

		// Complete an executed function.
		void CompleteAsyncFunction(
			const FExecutedAsyncFunction& ExecutedAsyncFunction,
			const FString& ResultJson, bool bResultJsonIsError);

		// Invoke a registered callback.
		void InvokeCallback(
			const FString& HandlerId, const FString& ResultJson, bool bResultJsonIsError);

	protected:
		void ExecuteAsyncFunction(
			const TCHAR* InstanceName, const TCHAR* FunctionName, const TCHAR* Arguments,
			const TCHAR* HandlerId) override;

		FString RegisterCallback(
			TFunction<UAIAssistantWebJavaScriptResultDelegate::EHandlerStatus(
				UAIAssistantWebJavaScriptResultDelegate::FResult)> Callback) override;

		// If possible, get the value of a result as JSON.  If the value cannot be converted to
		// JSON return an empty string.
		template<typename ValueType>
		FString GetValueOrEmptyString(const TValueOrError<ValueType, FString>& ValueOrError)
		{
			check(!ValueOrError.HasError());
			return ValueOrError.GetValue().ToJson(false);
		}

		// Return an empty string for the value of a void TValueOrError.
		template<>
		FString GetValueOrEmptyString<void>(const TValueOrError<void, FString>& ValueOrError)
		{
			check(!ValueOrError.HasError());
			return FString();
		}

	public:
		TArray<FString> RegisteredCallbackHandlerIds;
		TArray<FExecutedAsyncFunction> ExecutedAsyncFunctions;

	public:
		// Get the function used to determine whether the web API is available.
		static FString GetWebApiAvailableFunction() { return WebApiAvailableFunction; }
	};
}

#endif