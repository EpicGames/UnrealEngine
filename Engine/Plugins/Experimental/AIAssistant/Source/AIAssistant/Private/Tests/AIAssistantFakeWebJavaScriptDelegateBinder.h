// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "UObject/Object.h"
#include "UObject/StrongObjectPtr.h"

#include "AIAssistantLog.h"
#include "AIAssistantWebJavaScriptDelegateBinder.h"

namespace UE::AIAssistant
{
	// Fake JavaScript delegate binder.
	struct FFakeWebJavaScriptDelegateBinder : public IWebJavaScriptDelegateBinder
	{
		struct BoundObject
		{
			TStrongObjectPtr<UObject> Object;
			bool bIsPermanent;
		};

		virtual ~FFakeWebJavaScriptDelegateBinder() = default;

		void BindUObject(const FString& Name, UObject* Object, bool bIsPermanent = true) override
		{
			if (BoundObjects.Find(Name))
			{
				UE_LOGF(
					LogAIAssistant, Error,
					"Unable to bind object %ls as JavaScript delegate as it is already bound",
					*Name);
				return;
			}
			BoundObjects.Emplace(Name, BoundObject{ TStrongObjectPtr<UObject>(Object), bIsPermanent });
		}

		void UnbindUObject(const FString& Name, UObject* Object, bool bIsPermanent = true) override
		{
			auto* BoundObject = BoundObjects.Find(Name);
			if (!BoundObject)
			{
				UE_LOGF(
					LogAIAssistant, Error,
					"Object %ls was not bound as a JavaScript delegate", *Name);
				return;
			}
			if (BoundObject->bIsPermanent != bIsPermanent)
			{
				UE_LOGF(
					LogAIAssistant, Error,
					"Unable to unbind object %ls due to bIsPermanent mismatch %d vs %d",
					*Name, bIsPermanent ? 1 : 0, BoundObject->bIsPermanent ? 1 : 0);
				return;
			}
			BoundObjects.Remove(Name);
		}

		TMap<FString, BoundObject> BoundObjects;
	};
}