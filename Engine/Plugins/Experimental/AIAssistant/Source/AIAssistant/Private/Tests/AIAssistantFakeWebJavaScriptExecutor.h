// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"

#include "AIAssistantWebJavaScriptExecutor.h"

namespace UE::AIAssistant
{
	// Executes JavaScript in a web browser.
	struct FFakeWebJavaScriptExecutor : public IWebJavaScriptExecutor
	{	
		virtual ~FFakeWebJavaScriptExecutor() = default;

		void ExecuteJavaScript(const FString& JavaScriptText) override
		{
			ExecutedJavaScriptText.Add(JavaScriptText);
		}

		TArray<FString> ExecutedJavaScriptText;
	};
}