// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"

#include "AIAssistantTestFlags.h"
#include "AIAssistantFileLockManager.h"

#if WITH_DEV_AUTOMATION_TESTS

BEGIN_DEFINE_SPEC(FAIAssistantFileLockManagerTest,
	"AI.Assistant.FileLockManager",
	AIAssistantTest::Flags)
END_DEFINE_SPEC(FAIAssistantFileLockManagerTest)

void FAIAssistantFileLockManagerTest::Define()
{
	using namespace UE::AIAssistant;

	AfterEach([this]
		{
			FFileLockManager::ClearLockedFiles();
		});

	Describe("AddLockedFile", [this]
		{
			It("returns true when adding a new file", [this]
				{
					const bool bResult = FFileLockManager::AddLockedFile(TEXT("/Game/MyAsset"));
					TestTrue(TEXT("Should return true for new file"), bResult);
				});

			It("returns false when adding a duplicate file", [this]
				{
					FFileLockManager::AddLockedFile(TEXT("/Game/MyAsset"));
					const bool bResult = FFileLockManager::AddLockedFile(TEXT("/Game/MyAsset"));
					TestFalse(TEXT("Should return false for duplicate"), bResult);
				});

			It("returns false for empty path", [this]
				{
					const bool bResult = FFileLockManager::AddLockedFile(TEXT(""));
					TestFalse(TEXT("Should return false for empty path"), bResult);
				});

			It("normalizes paths without leading slash", [this]
				{
					FFileLockManager::AddLockedFile(TEXT("Game/MyAsset"));
					TestTrue(
						TEXT("Should be findable with leading slash"),
						FFileLockManager::IsFileLocked(TEXT("/Game/MyAsset")));
				});

			It("treats paths with and without leading slash as the same", [this]
				{
					FFileLockManager::AddLockedFile(TEXT("/Game/MyAsset"));
					const bool bResult = FFileLockManager::AddLockedFile(TEXT("Game/MyAsset"));
					TestFalse(TEXT("Should detect duplicate after normalization"), bResult);
				});
		});

	Describe("RemoveLockedFile", [this]
		{
			It("returns true when removing an existing file", [this]
				{
					FFileLockManager::AddLockedFile(TEXT("/Game/MyAsset"));
					const bool bResult = FFileLockManager::RemoveLockedFile(TEXT("/Game/MyAsset"));
					TestTrue(TEXT("Should return true for existing file"), bResult);
				});

			It("returns false when removing a non-existent file", [this]
				{
					const bool bResult = FFileLockManager::RemoveLockedFile(TEXT("/Game/MyAsset"));
					TestFalse(TEXT("Should return false for non-existent file"), bResult);
				});

			It("file is no longer locked after removal", [this]
				{
					FFileLockManager::AddLockedFile(TEXT("/Game/MyAsset"));
					FFileLockManager::RemoveLockedFile(TEXT("/Game/MyAsset"));
					TestFalse(
						TEXT("File should not be locked after removal"),
						FFileLockManager::IsFileLocked(TEXT("/Game/MyAsset")));
				});

			It("normalizes paths without leading slash", [this]
				{
					FFileLockManager::AddLockedFile(TEXT("/Game/MyAsset"));
					const bool bResult = FFileLockManager::RemoveLockedFile(TEXT("Game/MyAsset"));
					TestTrue(TEXT("Should remove via normalized path"), bResult);
				});
		});

	Describe("IsFileLocked", [this]
		{
			It("returns false for unlocked file", [this]
				{
					TestFalse(
						TEXT("Should return false for unlocked file"),
						FFileLockManager::IsFileLocked(TEXT("/Game/MyAsset")));
				});

			It("returns true for locked file", [this]
				{
					FFileLockManager::AddLockedFile(TEXT("/Game/MyAsset"));
					TestTrue(
						TEXT("Should return true for locked file"),
						FFileLockManager::IsFileLocked(TEXT("/Game/MyAsset")));
				});

			It("matches sub-assets via dot prefix", [this]
				{
					FFileLockManager::AddLockedFile(TEXT("/Game/MyAsset"));
					TestTrue(
						TEXT("Sub-asset should be locked"),
						FFileLockManager::IsFileLocked(TEXT("/Game/MyAsset.SubObject")));
				});

			It("does not match unrelated paths that share a prefix", [this]
				{
					FFileLockManager::AddLockedFile(TEXT("/Game/My"));
					TestFalse(
						TEXT("Should not match path that merely starts with the locked prefix"),
						FFileLockManager::IsFileLocked(TEXT("/Game/MyAsset")));
				});

			It("normalizes paths without leading slash", [this]
				{
					FFileLockManager::AddLockedFile(TEXT("/Game/MyAsset"));
					TestTrue(
						TEXT("Should find locked file without leading slash"),
						FFileLockManager::IsFileLocked(TEXT("Game/MyAsset")));
				});
		});

	Describe("GetLockedFiles", [this]
		{
			It("returns empty set when no files are locked", [this]
				{
					const TSet<FString> LockedFiles = FFileLockManager::GetLockedFiles();
					TestEqual(TEXT("Should have no locked files"), LockedFiles.Num(), 0);
				});

			It("returns all locked files", [this]
				{
					FFileLockManager::AddLockedFile(TEXT("/Game/AssetA"));
					FFileLockManager::AddLockedFile(TEXT("/Game/AssetB"));
					const TSet<FString> LockedFiles = FFileLockManager::GetLockedFiles();
					TestEqual(TEXT("Should have two locked files"), LockedFiles.Num(), 2);
					TestTrue(TEXT("Should contain AssetA"), LockedFiles.Contains(TEXT("/Game/AssetA")));
					TestTrue(TEXT("Should contain AssetB"), LockedFiles.Contains(TEXT("/Game/AssetB")));
				});
		});

	Describe("ClearLockedFiles", [this]
		{
			It("removes all locked files", [this]
				{
					FFileLockManager::AddLockedFile(TEXT("/Game/AssetA"));
					FFileLockManager::AddLockedFile(TEXT("/Game/AssetB"));
					FFileLockManager::ClearLockedFiles();
					TestEqual(
						TEXT("Should have no locked files after clear"),
						FFileLockManager::GetLockedFiles().Num(), 0);
				});

			It("previously locked files are no longer locked", [this]
				{
					FFileLockManager::AddLockedFile(TEXT("/Game/MyAsset"));
					FFileLockManager::ClearLockedFiles();
					TestFalse(
						TEXT("File should not be locked after clear"),
						FFileLockManager::IsFileLocked(TEXT("/Game/MyAsset")));
				});
		});
}

#endif  // WITH_DEV_AUTOMATION_TESTS
