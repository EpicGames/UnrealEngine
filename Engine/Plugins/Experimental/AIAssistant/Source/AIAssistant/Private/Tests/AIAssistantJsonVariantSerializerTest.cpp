// Copyright Epic Games, Inc. All Rights Reserved.

#include "Containers/UnrealString.h"
#include "Misc/AutomationTest.h"
#include "Misc/Variant.h"
#include "Serialization/JsonSerializable.h"
#include "Serialization/JsonSerializerMacros.h"

#include "AIAssistantEnum.h"
#include "AIAssistantJsonVariantSerializer.h"
#include "AIAssistantTestFlags.h"

#if WITH_DEV_AUTOMATION_TESTS

using namespace UE::AIAssistant;

enum class ESpellType
{
	Shrink,
	Vanish,
};

#define UE_AI_ASSISTANT_SPELL_TYPE_ENUM(X) \
	X(ESpellType::Shrink, "shrink"), \
	X(ESpellType::Vanish, "vanish")

UE_ENUM_METADATA_DECLARE(ESpellType, UE_AI_ASSISTANT_SPELL_TYPE_ENUM);
UE_ENUM_METADATA_DEFINE(ESpellType, UE_AI_ASSISTANT_SPELL_TYPE_ENUM);

struct FShrinkSpell : public FJsonSerializable
{
	FString Size;

	BEGIN_JSON_SERIALIZER
		JSON_SERIALIZE("size", Size);
	END_JSON_SERIALIZER
};

struct FVanishSpell : public FJsonSerializable
{
	int Duration;

	BEGIN_JSON_SERIALIZER
		JSON_SERIALIZE("duration", Duration);
	END_JSON_SERIALIZER
};

struct FSpell : public FJsonSerializable
{
	ESpellType SpellType;
	TVariant<FShrinkSpell, FVanishSpell> Spell;

	BEGIN_JSON_SERIALIZER
		UE_JSON_SERIALIZE_ENUM_VARIANT_BEGIN("spellType", SpellType, "spell", Spell);
			UE_JSON_SERIALIZE_ENUM_VARIANT(ESpellType::Shrink, FShrinkSpell);
			UE_JSON_SERIALIZE_ENUM_VARIANT(ESpellType::Vanish, FVanishSpell);
		UE_JSON_SERIALIZE_ENUM_VARIANT_END();
	END_JSON_SERIALIZER
};

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantJsonVariantSerializerTestSave,
	"AI.Assistant.JsonVariantSerializer.Save",
	AIAssistantTest::Flags);

bool FAIAssistantJsonVariantSerializerTestSave::RunTest(const FString& UnusedParameters)
{
	FSpell Spell;
	Spell.SpellType = ESpellType::Vanish;
	Spell.Spell.Emplace<FVanishSpell>();
	Spell.Spell.Get<FVanishSpell>().Duration = 42;
	return TestEqual(
		TEXT("Spell"),
		Spell.ToJson(false),
		TEXT(R"json({"spellType":"vanish","spell":{"duration":42}})json"));
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantJsonVariantSerializerTestLoad,
	"AI.Assistant.JsonVariantSerializer.Load",
	AIAssistantTest::Flags);

bool FAIAssistantJsonVariantSerializerTestLoad::RunTest(const FString& UnusedParameters)
{
	FSpell Spell;
	if (TestTrue(
		TEXT("Spell.FromJson"),
		Spell.FromJson(TEXT(R"json({"spellType":"shrink","spell":{"size":"ant"}})json"))))
	{
		(void)TestEqual(TEXT("Spell.SpellType"), Spell.SpellType, ESpellType::Shrink);
		if (TestTrue(TEXT("Spell.Spell"), Spell.Spell.IsType<FShrinkSpell>()))
		{
			(void)TestEqual(
				TEXT("Spell.Spell.Size"),
				Spell.Spell.Get<FShrinkSpell>().Size,
				TEXT("ant"));
		}
	}
	return true;
}

#endif  // WITH_DEV_AUTOMATION_TESTS