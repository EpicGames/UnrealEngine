// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/AIAssistantMessageUtils.h"

#include "AIAssistantWebApi.h"
#include "Containers/UnrealString.h"
#include "Misc/DateTime.h"
#include "Misc/Optional.h"
#include "Misc/Variant.h"
#include "Templates/Tuple.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::AIAssistant
{
	FMessage MakeMessage(int Index, EMessageRole MessageRole)
	{
		FMessage Message;
		Message.MessageId = MakeMessageId(Index);
		Message.MessageRole = MessageRole;
		return Message;
	}

	FMessage MakeTextMessage(int Index, EMessageRole MessageRole, const FString& Text)
	{
		FMessage Message = MakeMessage(Index, MessageRole);
		FMessageContent& MessageContent = Message.MessageContent.Emplace_GetRef();
		MessageContent.ContentType = EMessageContentType::Text;
		MessageContent.bVisibleToUser = true;
		MessageContent.Content.Emplace<FTextMessageContent>();
		MessageContent.Content.Get<FTextMessageContent>().Text = Text;
		return Message;
	}

	FConversationId MakeConversationId(int Index)
	{
		FConversationId ConversationId;
		ConversationId.Id = FString::Printf(TEXT("Conversation%i"), Index);
		ConversationId.Type = TEXT("ConversationId");
		return ConversationId;
	}

	FMessageId MakeMessageId(int Index)
	{
		FMessageId MessageId;
		MessageId.Id = FString::Printf(TEXT("Message%i"), Index);
		MessageId.Type = TEXT("Message");
		return MessageId;
	}

	FMessage MakeAgentTextMessage(int Index, const FString& Text)
	{
		return MakeTextMessage(Index, EMessageRole::Agent, Text);
	}

	FString MakeToolCallId(int MessageIndex, int ToolCallIndex)
	{
		return FString::Printf(TEXT("ToolCall%i-%i"), MessageIndex, ToolCallIndex);
	}

	FToolCallContent MakeToolCallContent(
		FString Name,
		FString ArgumentsRawJson,
		TOptional<TTuple<int, int>> MessageAndToolCallIndex)
	{
		FToolCallContent ToolCallContent;
		ToolCallContent.Name = Name;
		ToolCallContent.ArgumentsRawJson = ArgumentsRawJson;
		if (MessageAndToolCallIndex.IsSet())
		{
			const auto& [MessageIndex, ToolCallIndex] = MessageAndToolCallIndex.GetValue();
			ToolCallContent.ToolCallId = MakeToolCallId(MessageIndex, ToolCallIndex);
			ToolCallContent.ResponseRequired = true;
		}
		return ToolCallContent;
	}

	FMessage MakeAgentToolCallMessage(int Index, const TArray<FToolCallContent>& ToolCalls)
	{
		FMessage Message;
		Message.MessageRole = EMessageRole::Agent;
		Message.MessageId = MakeMessageId(Index);
		for (const auto& ToolCallContent : ToolCalls)
		{
			FMessageContent& MessageContent = Message.MessageContent.Emplace_GetRef();
			MessageContent.ContentType = EMessageContentType::ToolCall;
			MessageContent.bVisibleToUser = false;
			MessageContent.Content.Emplace<FToolCallContent>(ToolCallContent);
		}
		return Message;
	}

	FMessage MakeUserTextMessage(int Index, const FString& Text)
	{
		return MakeTextMessage(Index, EMessageRole::User, Text);
	}

	FToolResponseContent MakeToolResponseContent(
		FString ToolCallId,
		FString Name,
		FString ResponseRawJson,
		TOptional<bool> Success,
		TOptional<FString> ErrorMessage)
	{
		FToolResponseContent Response;
		Response.ToolCallId = ToolCallId;
		Response.Name = Name;
		Response.ResponseRawJson = ResponseRawJson;
		Response.Success = Success;
		Response.ErrorMessage = ErrorMessage;
		return Response;
	}

	FMessage MakeUserToolResponseMessage(
		int Index, const TArray<FToolResponseContent>& ToolResponses)
	{
		FMessage Message = MakeMessage(Index, EMessageRole::User);
		for (const FToolResponseContent& ToolResponse : ToolResponses)
		{
			FMessageContent& MessageContent = Message.MessageContent.Emplace_GetRef();
			MessageContent.ContentType = EMessageContentType::ToolResponse;
			MessageContent.Content.Emplace<FToolResponseContent>(ToolResponse);
		}
		return Message;
	}

	FString MakeConversationUpdateEventJson(const TCHAR* ConversationId)
	{
		return MakeConversationUpdateEventJson(
			ConversationId, EConversationUpdateType::MessagesUpdated);
	}

	FString MakeConversationUpdateEventJson(
		const TCHAR* ConversationId,
		EConversationUpdateType UpdateType)
	{
		return MakeConversationUpdateEventJson(ConversationId, *LexToString(UpdateType));
	}

	FString MakeConversationUpdateEventJson(
		const TCHAR* ConversationId,
		const TCHAR* UpdateTypeString)
	{
		return FString::Printf(
			TEXT(R"js({"conversationId":{"id":"%s","type":"ConversationId"},"updateType":"%s"})js"),
			ConversationId, UpdateTypeString);
	}
}

#endif  // WITH_DEV_AUTOMATION_TESTS
