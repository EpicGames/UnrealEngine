// Copyright Epic Games, Inc. All Rights Reserved.

#include "AIAssistantWebApi.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::AIAssistant
{
	FMessage MakeTextMessage(int Index, EMessageRole MessageRole, const FString& Text);
	
	FConversationId MakeConversationId(int Index);

	FMessageId MakeMessageId(int Index);

	FMessage MakeMessage(int Index, EMessageRole MessageRole);

	FMessage MakeAgentTextMessage(int Index, const FString& Text);

	FString MakeToolCallId(int MessageIndex, int ToolCallIndex);

	FToolCallContent MakeToolCallContent(
		FString Name,
		FString ArgumentsRawJson,
		TOptional<TTuple<int, int>> MessageAndToolCallIndex);

	FMessage MakeAgentToolCallMessage(int Index, const TArray<FToolCallContent>& ToolCalls);

	FMessage MakeUserTextMessage(int Index, const FString& Text);

	FToolResponseContent MakeToolResponseContent(
		FString ToolCallId,
		FString Name,
		FString ResponseRawJson,
		TOptional<bool> Success,
		TOptional<FString> ErrorMessage);

	FMessage MakeUserToolResponseMessage(
		int Index, const TArray<FToolResponseContent>& ToolResponses);

	// Build the JSON payload sent by the frontend for a conversation update event.
	// The enum overload is preferred for known update types; the string overload
	// is for tests that deliberately send an unrecognized type string.
	FString MakeConversationUpdateEventJson(const TCHAR* ConversationId);
	FString MakeConversationUpdateEventJson(
		const TCHAR* ConversationId,
		EConversationUpdateType UpdateType);

	FString MakeConversationUpdateEventJson(
		const TCHAR* ConversationId,
		const TCHAR* UpdateTypeString);
}

#endif  // DEV_AUTOMATION_TESTS
