// Copyright Epic Games, Inc. All Rights Reserved.

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Misc/AutomationTest.h"

#include "AIAssistantRunSequential.h"
#include "Tests/AIAssistantTestFlags.h"

#if WITH_DEV_AUTOMATION_TESTS

using UE::AIAssistant::RunSequential;

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantWebApplicationRunSequentialSuccess,
	"AI.Assistant.RunSequential.Success",
	AIAssistantTest::Flags);

bool FAIAssistantWebApplicationRunSequentialSuccess::RunTest(const FString& UnusedParameters)
{
	// Track the order and values of lambda invocations.
	TArray<int32> TestInputs = { 1, 2, 3, 4, 5 };
	TArray<int32> ProcessedValues;
	auto Result = RunSequential(
		TestInputs,
		[&ProcessedValues](int32 Value) -> TFuture<void>
		{
			ProcessedValues.Add(Value);
			return MakeFulfilledPromise<void>().GetFuture();
		});

	Result.Get();
	if (!TestEqual(TEXT("All inputs should be processed"), ProcessedValues.Num(), TestInputs.Num()))
	{
		return false;
	}

	// Validate that lambdas were called in the correct sequential order.
	for (int32 i = 0; i < TestInputs.Num(); ++i)
	{
		TestEqual(
			FString::Printf(TEXT("Input %d should be processed in order"), i),
			ProcessedValues[i],
			TestInputs[i]);
	}
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantWebApplicationRunSequentialFailure,
	"AI.Assistant.RunSequential.Failure",
	AIAssistantTest::Flags);
bool FAIAssistantWebApplicationRunSequentialFailure::RunTest(
	const FString& UnusedParameters)
{
	// Predicate that fails on value 3. RunSequential continues unconditionally,
	// so all items are processed regardless of earlier failures.
	TArray<int32> TestInputs = { 1, 2, 3, 4, 5 };
	TArray<int32> SucceededValues;
	TArray<int32> FailedValues;
	auto Result = RunSequential(
		TestInputs,
		[&SucceededValues, &FailedValues](int32 Value) -> TFuture<void>
		{
			if (Value == 3)
			{
				FailedValues.Add(Value);
				return MakeFulfilledPromise<void>().GetFuture();
			}
			SucceededValues.Add(Value);
			return MakeFulfilledPromise<void>().GetFuture();
		});

	Result.Get();

	// All items should have been processed (no abort on failure).
	if (!TestEqual(
		TEXT("All successful values should be processed"),
		SucceededValues.Num(), 4))
	{
		return false;
	}
	TestEqual(TEXT("Value 1 should succeed"), SucceededValues[0], 1);
	TestEqual(TEXT("Value 2 should succeed"), SucceededValues[1], 2);
	TestEqual(TEXT("Value 4 should succeed"), SucceededValues[2], 4);
	TestEqual(TEXT("Value 5 should succeed"), SucceededValues[3], 5);

	if (!TestEqual(TEXT("Only one value should fail"), FailedValues.Num(), 1))
	{
		return false;
	}
	TestEqual(TEXT("Value 3 should fail"), FailedValues[0], 3);

	return true;
}

#endif
