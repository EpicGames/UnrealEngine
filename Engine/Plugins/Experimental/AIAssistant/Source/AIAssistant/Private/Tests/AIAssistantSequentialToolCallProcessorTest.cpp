// Copyright Epic Games, Inc. All Rights Reserved.

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Misc/AutomationTest.h"
#include "Templates/SharedPointer.h"

#include "AIAssistantMessageUtils.h"
#include "AIAssistantToolResponse.h"
#include "AIAssistantWebApi.h"
#include "AIAssistantWebApplication.h"
#include "Tests/AIAssistantTestFlags.h"

#if WITH_DEV_AUTOMATION_TESTS

using namespace UE::AIAssistant;

// Minimal spy that records ProcessToolCallContent and AddUserMessageToConversation
// calls without routing through a real WebApi.
class FToolCallProcessorSpyWebApplication : public FWebApplication
{
public:
	using FWebApplication::FWebApplication;
	using FWebApplication::ProcessMessage;
	using FWebApplication::OnStopGenerating;

	void AddUserMessageToConversation(
		FAddMessageToConversationOptions&& Options) override
	{
		AddMessageCalls.Add(MoveTemp(Options));
	}

	TFuture<TValueOrError<void, FString>> ProcessToolCallContent(
		const FToolCallContent& ToolCall, const FString& ConversationId) override
	{
		ProcessedToolCalls.Add(ToolCall);
		TSharedPtr<TPromise<TValueOrError<void, FString>>> ResultPromise;
		if (FakeResults.Dequeue(ResultPromise))
		{
			return ResultPromise->GetFuture();
		}
		return MakeFulfilledPromise<TValueOrError<void, FString>>(MakeValue()).GetFuture();
	}

	TArray<FToolCallContent> ProcessedToolCalls;
	TArray<FAddMessageToConversationOptions> AddMessageCalls;
	TQueue<TSharedPtr<TPromise<TValueOrError<void, FString>>>> FakeResults;
};

static TSharedPtr<FToolCallProcessorSpyWebApplication> CreateSpy()
{
	auto Factory = []() -> TSharedPtr<FWebApi> { return nullptr; };
	return MakeShared<FToolCallProcessorSpyWebApplication>(MoveTemp(Factory));
}

static void EnqueueSuccess(TSharedPtr<FToolCallProcessorSpyWebApplication>& Spy)
{
	Spy->FakeResults.Enqueue(
		MakeShared<TPromise<TValueOrError<void, FString>>>(
			MakeFulfilledPromise<TValueOrError<void, FString>>(MakeValue())));
}

static TSharedPtr<TPromise<TValueOrError<void, FString>>> EnqueuePending(
	TSharedPtr<FToolCallProcessorSpyWebApplication>& Spy)
{
	auto Promise = MakeShared<TPromise<TValueOrError<void, FString>>>();
	Spy->FakeResults.Enqueue(Promise);
	return Promise;
}

// --------------------------------------------------------------------------
// Tests
// --------------------------------------------------------------------------

BEGIN_DEFINE_SPEC(
	FAIAssistantSequentialToolCallProcessorTest,
	"AI.Assistant.SequentialToolCallProcessor",
	AIAssistantTest::Flags)
END_DEFINE_SPEC(FAIAssistantSequentialToolCallProcessorTest)

void FAIAssistantSequentialToolCallProcessorTest::Define()
{
	Describe(TEXT("when all tool calls succeed"), [this]
	{
		It(TEXT("should process every tool call in order"), [this]
		{
			auto Spy = CreateSpy();
			FToolCallContent A = MakeToolCallContent(
				TEXT("ToolA"), TEXT("{}"), MakeTuple(1, 1));
			FToolCallContent B = MakeToolCallContent(
				TEXT("ToolB"), TEXT("{}"), MakeTuple(1, 2));
			FToolCallContent C = MakeToolCallContent(
				TEXT("ToolC"), TEXT("{}"), MakeTuple(1, 3));
			FMessage Message = MakeAgentToolCallMessage(1, { A, B, C });

			EnqueueSuccess(Spy);
			EnqueueSuccess(Spy);
			EnqueueSuccess(Spy);

			Spy->ProcessMessage(Message, TEXT("conv"));

			if (!TestEqual(TEXT("AllProcessed"),
				Spy->ProcessedToolCalls.Num(), 3))
			{
				return;
			}
			(void)TestEqual(TEXT("FirstTool"),
				Spy->ProcessedToolCalls[0].Name, TEXT("ToolA"));
			(void)TestEqual(TEXT("SecondTool"),
				Spy->ProcessedToolCalls[1].Name, TEXT("ToolB"));
			(void)TestEqual(TEXT("ThirdTool"),
				Spy->ProcessedToolCalls[2].Name, TEXT("ToolC"));
			(void)TestEqual(TEXT("NoAbortMessages"),
				Spy->AddMessageCalls.Num(), 0);
		});
	});

	Describe(TEXT("when a tool call fails"), [this]
	{
		It(TEXT("should abort remaining tool calls after the failure"), [this]
		{
			auto Spy = CreateSpy();
			FToolCallContent A = MakeToolCallContent(
				TEXT("ToolA"), TEXT("{}"), MakeTuple(1, 1));
			FToolCallContent B = MakeToolCallContent(
				TEXT("ToolB"), TEXT("{}"), MakeTuple(1, 2));
			FToolCallContent C = MakeToolCallContent(
				TEXT("ToolC"), TEXT("{}"), MakeTuple(1, 3));
			FMessage Message = MakeAgentToolCallMessage(1, { A, B, C });

			// A succeeds, B fails.
			EnqueueSuccess(Spy);
			auto FailPromise = EnqueuePending(Spy);

			Spy->ProcessMessage(Message, TEXT("conv"));
			FailPromise->SetValue(MakeError(TEXT("ToolB exploded")));

			// Only A and B should have been processed. C should be aborted.
			(void)TestEqual(TEXT("ProcessedCount"),
				Spy->ProcessedToolCalls.Num(), 2);
			(void)TestEqual(TEXT("FirstProcessed"),
				Spy->ProcessedToolCalls[0].Name, TEXT("ToolA"));
			(void)TestEqual(TEXT("SecondProcessed"),
				Spy->ProcessedToolCalls[1].Name, TEXT("ToolB"));

			// C should receive an abort response message.
			if (!TestEqual(TEXT("AbortMessageCount"),
				Spy->AddMessageCalls.Num(), 1))
			{
				return;
			}
			FString CToolCallId = MakeToolCallId(1, 3);
			auto ExpectedAbort = CreateToolResponseMessage(
				C.Name, CToolCallId,
				MakeError(FString(TEXT("Tool call aborted due to previous failure"))));
			(void)TestEqual(TEXT("AbortResponseContent"),
				Spy->AddMessageCalls[0].ToJson(true),
				ExpectedAbort.ToJson(true));
		});

		It(TEXT("should abort all tool calls after the first failure"), [this]
		{
			auto Spy = CreateSpy();
			FToolCallContent A = MakeToolCallContent(
				TEXT("ToolA"), TEXT("{}"), MakeTuple(1, 1));
			FToolCallContent B = MakeToolCallContent(
				TEXT("ToolB"), TEXT("{}"), MakeTuple(1, 2));
			FToolCallContent C = MakeToolCallContent(
				TEXT("ToolC"), TEXT("{}"), MakeTuple(1, 3));
			FMessage Message = MakeAgentToolCallMessage(1, { A, B, C });

			// A fails immediately.
			auto FailPromise = EnqueuePending(Spy);

			Spy->ProcessMessage(Message, TEXT("conv"));
			FailPromise->SetValue(MakeError(TEXT("ToolA exploded")));

			// Only A should have been processed.
			(void)TestEqual(TEXT("ProcessedCount"),
				Spy->ProcessedToolCalls.Num(), 1);
			(void)TestEqual(TEXT("FirstProcessed"),
				Spy->ProcessedToolCalls[0].Name, TEXT("ToolA"));

			// B and C should both receive abort responses.
			if (!TestEqual(TEXT("AbortMessageCount"),
				Spy->AddMessageCalls.Num(), 2))
			{
				return;
			}
			FAddMessageToConversationOptions ExpectedAbortB = CreateToolResponseMessage(
				B.Name, MakeToolCallId(1, 2),
				MakeError(FString(TEXT("Tool call aborted due to previous failure"))));
			(void)TestEqual(TEXT("BAbortContent"),
				Spy->AddMessageCalls[0].ToJson(true),
				ExpectedAbortB.ToJson(true));
			FAddMessageToConversationOptions ExpectedAbortC = CreateToolResponseMessage(
				C.Name, MakeToolCallId(1, 3),
				MakeError(FString(TEXT("Tool call aborted due to previous failure"))));
			(void)TestEqual(TEXT("CAbortContent"),
				Spy->AddMessageCalls[1].ToJson(true),
				ExpectedAbortC.ToJson(true));
		});
	});

	Describe(TEXT("when the WebApplication is destroyed"), [this]
	{
		It(TEXT("should complete gracefully without processing remaining calls"), [this]
		{
			auto Spy = CreateSpy();
			FToolCallContent A = MakeToolCallContent(
				TEXT("ToolA"), TEXT("{}"), MakeTuple(1, 1));
			FToolCallContent B = MakeToolCallContent(
				TEXT("ToolB"), TEXT("{}"), MakeTuple(1, 2));
			FMessage Message = MakeAgentToolCallMessage(1, { A, B });

			// Leave A's promise unfulfilled so we can destroy the spy first.
			auto PendingPromise = EnqueuePending(Spy);

			Spy->ProcessMessage(Message, TEXT("conv"));

			// Destroy the WebApplication before fulfilling the promise.
			Spy.Reset();

			// Fulfilling after destruction should not crash.
			PendingPromise->SetValue(MakeValue());

			// If we reach here without crashing, the test passes.
			(void)TestTrue(TEXT("CompletedWithoutCrash"), true);
		});
	});

	Describe(TEXT("when user stops generation"), [this]
	{
		It(TEXT("should abort all tool calls with user cancellation message"), [this]
		{
			auto Spy = CreateSpy();
			FToolCallContent A = MakeToolCallContent(
				TEXT("ToolA"), TEXT("{}"), MakeTuple(1, 1));
			FToolCallContent B = MakeToolCallContent(
				TEXT("ToolB"), TEXT("{}"), MakeTuple(1, 2));
			FToolCallContent C = MakeToolCallContent(
				TEXT("ToolC"), TEXT("{}"), MakeTuple(1, 3));
			FMessage Message = MakeAgentToolCallMessage(1, { A, B, C });

			EnqueueSuccess(Spy);
			EnqueueSuccess(Spy);
			EnqueueSuccess(Spy);

			// Cancel the conversation before processing.
			FConversationId StopConversationId;
			StopConversationId.Id = TEXT("conv");
			StopConversationId.Type = TEXT("ConversationId");
			Spy->OnStopGenerating(StopConversationId);

			Spy->ProcessMessage(Message, TEXT("conv"));

			// No tool calls should have been processed.
			(void)TestEqual(TEXT("ProcessedCount"),
				Spy->ProcessedToolCalls.Num(), 0);

			// All three should receive user cancellation abort responses.
			if (!TestEqual(TEXT("AbortMessageCount"),
				Spy->AddMessageCalls.Num(), 3))
			{
				return;
			}
			FString AToolCallId = MakeToolCallId(1, 1);
			auto ExpectedAbort = CreateToolResponseMessage(
				A.Name, AToolCallId,
				MakeError(FString(TEXT("Tool call aborted: user stopped generation"))));
			(void)TestEqual(TEXT("AbortResponseContent"),
				Spy->AddMessageCalls[0].ToJson(true),
				ExpectedAbort.ToJson(true));
		});

		It(TEXT("should abort subsequent tool calls after mid-flight cancellation"), [this]
		{
			auto Spy = CreateSpy();
			FToolCallContent A = MakeToolCallContent(TEXT("ToolA"), TEXT("{}"), MakeTuple(1, 1));
			FToolCallContent B = MakeToolCallContent(TEXT("ToolB"), TEXT("{}"), MakeTuple(1, 2));
			FToolCallContent C = MakeToolCallContent(TEXT("ToolC"), TEXT("{}"), MakeTuple(1, 3));
			FMessage Message = MakeAgentToolCallMessage(1, { A, B, C });

			// A succeeds immediately, B is pending.
			EnqueueSuccess(Spy);
			auto PendingPromise = EnqueuePending(Spy);

			Spy->ProcessMessage(Message, TEXT("conv"));

			// Cancel the conversation while B is pending.
			FConversationId StopConversationId;
			StopConversationId.Id = TEXT("conv");
			StopConversationId.Type = TEXT("ConversationId");
			Spy->OnStopGenerating(StopConversationId);

			// Complete B successfully.
			PendingPromise->SetValue(MakeValue());

			// A and B should have been processed.
			(void)TestEqual(TEXT("ProcessedCount"), Spy->ProcessedToolCalls.Num(), 2);
			(void)TestEqual(TEXT("FirstProcessed"), Spy->ProcessedToolCalls[0].Name, TEXT("ToolA"));
			(void)TestEqual(
				TEXT("SecondProcessed"), Spy->ProcessedToolCalls[1].Name, TEXT("ToolB"));

			// C should receive user cancellation abort response.
			if (!TestEqual(TEXT("AbortMessageCount"), Spy->AddMessageCalls.Num(), 1))
			{
				return;
			}
			FString CToolCallId = MakeToolCallId(1, 3);
			auto ExpectedAbort = CreateToolResponseMessage(
				C.Name, CToolCallId,
				MakeError(FString(TEXT("Tool call aborted: user stopped generation"))));
			(void)TestEqual(TEXT("AbortResponseContent"),
				Spy->AddMessageCalls[0].ToJson(true),
				ExpectedAbort.ToJson(true));
		});

		It(TEXT("should not affect tool calls for a different conversation"), [this]
		{
			auto Spy = CreateSpy();
			FToolCallContent A = MakeToolCallContent(
				TEXT("ToolA"), TEXT("{}"), MakeTuple(1, 1));
			FToolCallContent B = MakeToolCallContent(
				TEXT("ToolB"), TEXT("{}"), MakeTuple(1, 2));
			FMessage Message = MakeAgentToolCallMessage(1, { A, B });

			EnqueueSuccess(Spy);
			EnqueueSuccess(Spy);

			// Cancel a different conversation.
			FConversationId StopConversationId;
			StopConversationId.Id = TEXT("other-conv");
			StopConversationId.Type = TEXT("ConversationId");
			Spy->OnStopGenerating(StopConversationId);

			Spy->ProcessMessage(Message, TEXT("conv"));

			// Both tool calls for "conv" should have been processed normally.
			(void)TestEqual(TEXT("AllProcessed"), Spy->ProcessedToolCalls.Num(), 2);
			(void)TestEqual(TEXT("FirstTool"), Spy->ProcessedToolCalls[0].Name, TEXT("ToolA"));
			(void)TestEqual(TEXT("SecondTool"),Spy->ProcessedToolCalls[1].Name, TEXT("ToolB"));
			(void)TestEqual(TEXT("NoAbortMessages"), Spy->AddMessageCalls.Num(), 0);
		});
	});
}

#endif
