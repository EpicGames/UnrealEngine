// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"

#include "AIAssistantSlateQuerierTestHarness.h"

#if WITH_DEV_AUTOMATION_TESTS

// NOTE: Currently does not fully cover all the SlateQuerier functionality, as that will be refactored heavily
// in the near future to be more granular.

using namespace UE::AIAssistant::SlateQuerier;

class SlateQuerierTestWidget : public STestWidget
{
  public:
	void Construct(const FArguments& InArgs)
	{
		// TODO: add more complex constructions for more complex testing.
		ChildSlot
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SAssignNew(OuterHBox, SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SAssignNew(InnerHBox, SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.AutoWidth()
							[
								SAssignNew(TextBlock, STextBlock)
								.Text(FText::FromString(TEXT("Hello, Slate!")))
							 ]
						 ]
						+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							SAssignNew(CheckBoxButton, SCheckBox)
							.Type(ESlateCheckBoxType::ToggleButton)
							[
								SNew(STextBlock)
								.Text(FText::FromString(TEXT("CheckBox Name")))
							 ]
						 ]
						+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							SAssignNew(SimpleButton, SButton)
							.Text(FText::FromString(TEXT("Button Name")))
						 ]
					 ]
				 ]
			 ];
	}
};


IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantSlateQuerierTestCreateWidgetPath,
	"AI.Assistant.SlateQuerier.CreateWidgetPath",
	SlateQuerierTestFlags);

bool FAIAssistantSlateQuerierTestCreateWidgetPath::RunTest(const FString& UnusedParameters)
{
	FSlateTestHarness TestHarness(SNew(SlateQuerierTestWidget));

	TestHarness.GenerateWidgetPathTo(TestHarness.TestWidget);
	TestTrue(TEXT("WidgetPath is valid"), TestHarness.WidgetPath.IsValid());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantSlateQuerierTestItemIsButton,
	"AI.Assistant.SlateQuerier.ItemIsButton",
	SlateQuerierTestFlags);

bool FAIAssistantSlateQuerierTestItemIsButton::RunTest(const FString& UnusedParameters)
{
	FSlateTestHarness TestHarness(SNew(SlateQuerierTestWidget));

	TestHarness.GenerateWidgetPathTo(TestHarness.TestWidget->SimpleButton);
	TestTrue(TEXT("Identify SimpleButton"), TestHarness.SlateQuerier->TestItemIsButton());
	TestHarness.GenerateWidgetPathTo(TestHarness.TestWidget->CheckBoxButton);
	TestTrue(TEXT("Identify CheckBox"), TestHarness.SlateQuerier->TestItemIsButton());
	TestHarness.GenerateWidgetPathTo(TestHarness.TestWidget->TextBlock);
	TestFalse(TEXT("Text is not Button"), TestHarness.SlateQuerier->TestItemIsButton());

	return true;
}

#endif	// WITH_DEV_AUTOMATION_TESTS
