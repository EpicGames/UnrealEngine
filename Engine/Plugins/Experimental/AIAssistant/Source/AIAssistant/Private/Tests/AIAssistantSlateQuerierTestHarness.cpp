// Copyright Epic Games, Inc. All Rights Reserved.

#include "AIAssistantSlateQuerierTestHarness.h"

#if WITH_DEV_AUTOMATION_TESTS

#include "Widgets/SCompoundWidget.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "Framework/Application/SlateApplication.h"

#include "AIAssistantLog.h"

UE::AIAssistant::SlateQuerier::FSlateTestHarness::FSlateTestHarness(TSharedRef<STestWidget> InWidget)
{
	TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(FText::FromString(TEXT("Test Slate Window")))
		.ClientSize(FVector2D(400, 200));

	FSlateApplication::Get().AddWindow(Window);
	Window->SetContent(InWidget);
	TestWidget = InWidget.ToSharedPtr();
	ThisWindow = Window.ToSharedPtr();
}

UE::AIAssistant::SlateQuerier::FSlateTestHarness::~FSlateTestHarness()
{
	if (ThisWindow)
	{
		FSlateApplication::Get().RequestDestroyWindow(ThisWindow.ToSharedRef());
	}
}

void UE::AIAssistant::SlateQuerier::FSlateTestHarness::GenerateWidgetPathTo(const TSharedPtr<SWidget> InWidget)
{
	FSlateApplication::Get().GeneratePathToWidgetUnchecked(InWidget.ToSharedRef(), WidgetPath);
	FTestSlateQuerier* Querier = new FTestSlateQuerier(WidgetPath);
	SlateQuerier = TSharedPtr<FTestSlateQuerier>(Querier);
}

void UE::AIAssistant::SlateQuerier::FSlateTestHarness::LogWidgetPath()
{
	for (int32 WidgetIndex = 0; WidgetIndex < WidgetPath.Widgets.Num(); WidgetIndex++)
	{
		const FArrangedWidget* ArrangedWidget = &WidgetPath.Widgets[WidgetIndex];
		TSharedPtr<SWidget> ThisWidget = ArrangedWidget->Widget.ToSharedPtr();
		FName WidgetTypeName = ThisWidget->GetType();
		UE_LOGF(LogAIAssistant, Display, "Widget %d: %ls: %p",
			   WidgetIndex, *(WidgetTypeName.ToString()), ThisWidget.Get());
	}
}

void UE::AIAssistant::SlateQuerier::STestWidget::Construct(const FArguments& InArgs)
{
	ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SAssignNew(OuterHBox, SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SAssignNew(InnerHBox, SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							SAssignNew(TextBlock, STextBlock)
							.Text(FText::FromString(TEXT("Hello, Slate!")))
						 ]
					 ]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SAssignNew(CheckBoxButton, SCheckBox)
						.Type(ESlateCheckBoxType::ToggleButton)
						[
							SNew(STextBlock)
							.Text(FText::FromString(TEXT("CheckBox Name")))
						 ]
					 ]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SAssignNew(SimpleButton, SButton)
						.Text(FText::FromString(TEXT("Button Name")))
					 ]
				 ]
			 ]
		 ];
}

#endif  // WITH_DEV_AUTOMATION_TESTS
