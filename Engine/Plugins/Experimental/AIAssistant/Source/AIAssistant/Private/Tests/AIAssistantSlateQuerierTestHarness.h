// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_DEV_AUTOMATION_TESTS

#include "Widgets/SCompoundWidget.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Text/STextBlock.h"

#include "AIAssistantSlateQuerier.h"
#include "AIAssistantTestFlags.h"


namespace UE::AIAssistant::SlateQuerier
{
	const auto SlateQuerierTestFlags = AIAssistantTest::Flags | EAutomationTestFlags::NonNullRHI;

	class STestWidget : public SCompoundWidget
	{
	  public:
		SLATE_BEGIN_ARGS(STestWidget) {}
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs);

	  public:
		TSharedPtr<SHorizontalBox> OuterHBox;
		TSharedPtr<SHorizontalBox> InnerHBox;
		TSharedPtr<SButton> SimpleButton;
		TSharedPtr<SCheckBox> CheckBoxButton;
		TSharedPtr<STextBlock> TextBlock;
	};

	class FTestSlateQuerier : public FSlateQuerier
	{
	  public:
		bool TestItemIsButton() {return ItemIsButton();}
	};

	struct FSlateTestHarness
	{
		TSharedPtr<SWindow> ThisWindow;
		TSharedPtr<STestWidget> TestWidget;
		TSharedPtr<FTestSlateQuerier> SlateQuerier;
		FWidgetPath WidgetPath;

		FSlateTestHarness(TSharedRef<STestWidget> InWidget);
		~FSlateTestHarness();
		void GenerateWidgetPathTo(const TSharedPtr<SWidget> InWidget);
		void LogWidgetPath();
	};
};

#endif  // WITH_DEV_AUTOMATION_TESTS
