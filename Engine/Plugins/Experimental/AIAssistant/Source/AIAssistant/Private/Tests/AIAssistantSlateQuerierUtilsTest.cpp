// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"

#include "AIAssistantSlateQuerierTestHarness.h"

#include "AIAssistantSlateQuerierUtils.h"

#if WITH_DEV_AUTOMATION_TESTS

using namespace UE::AIAssistant::SlateQuerier;

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantSlateQuerierTestFindChildText,
	"AI.Assistant.SlateQuerier.Utils.FindChildText",
	SlateQuerierTestFlags);

bool FAIAssistantSlateQuerierTestFindChildText::RunTest(const FString& UnusedParameters)
{
	FSlateTestHarness TestHarness(SNew(STestWidget));

	TestHarness.GenerateWidgetPathTo(TestHarness.TestWidget);
	FText FoundText = Utility::FindChildWidgetWithText(TestHarness.TestWidget);
	TestFalse(TEXT("Find non-empty child text in widget"), FoundText.IsEmpty());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantSlateQuerierTestItemIsText,
	"AI.Assistant.SlateQuerier.Utils.ItemIsText",
	SlateQuerierTestFlags);

bool FAIAssistantSlateQuerierTestItemIsText::RunTest(const FString& UnusedParameters)
{
	FSlateTestHarness TestHarness(SNew(STestWidget));

	TestHarness.GenerateWidgetPathTo(TestHarness.TestWidget->TextBlock);
	FText FoundText = Utility::FindTextUnderCursor(TestHarness.WidgetPath);
	TestFalse(TEXT("Identify text block under cursor"), FoundText.IsEmpty());

	return true;
}

#endif	// WITH_DEV_AUTOMATION_TESTS
