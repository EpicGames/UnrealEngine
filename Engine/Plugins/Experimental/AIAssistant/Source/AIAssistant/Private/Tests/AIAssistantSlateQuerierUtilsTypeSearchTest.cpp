// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"

#include "AIAssistantLog.h"
#include "AIAssistantSlateQuerierTestHarness.h"
#include "AIAssistantSlateQuerierUtilsTypeSearch.h"

#if WITH_DEV_AUTOMATION_TESTS

// NOTE: Currently does not fully cover all the type-searching Utility functionality, that will be coming
// soon, along with more test UI harness infrastructure for testing more complex Slate UI such as Menus and
// Toolbars.

using namespace UE::AIAssistant::SlateQuerier;

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantSlateQuerierTestFindChildWidget,
	"AI.Assistant.SlateQuerier.Utils.TypeSearch.FindChildWidget",
	SlateQuerierTestFlags);

bool FAIAssistantSlateQuerierTestFindChildWidget::RunTest(const FString& UnusedParameters)
{
	FSlateTestHarness TestHarness(SNew(STestWidget));

	TSharedPtr<SWidget> ChildWidget = Utility::FindChildWidgetOfType(
		TestHarness.TestWidget->SimpleButton, "STextBlock");
	TestTrue(TEXT("Button's child text widget found"), ChildWidget.IsValid());
	TestTrue(TEXT("Button's child text is text"), ChildWidget->GetType() == "STextBlock");

	ChildWidget = Utility::FindChildWidgetOfType(TestHarness.TestWidget->TextBlock, "SButton");
	TestFalse(TEXT("Text does not have child Button widget"), ChildWidget.IsValid());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantSlateQuerierTestFindChildWidgets,
	"AI.Assistant.SlateQuerier.Utils.TypeSearch.FindChildWidgets",
	SlateQuerierTestFlags);

bool FAIAssistantSlateQuerierTestFindChildWidgets::RunTest(const FString& UnusedParameters)
{
	FSlateTestHarness TestHarness(SNew(STestWidget));

	TArray<TSharedRef<SWidget>> FoundWidgets;

	Utility::FindChildWidgetsOfType(FoundWidgets, TestHarness.TestWidget, "STextBlock");
	TestTrue(TEXT("Find multiple text widgets"), FoundWidgets.Num() > 1);

	FoundWidgets.Empty();
	Utility::FindChildWidgetsOfType(FoundWidgets, TestHarness.TestWidget, "SCheckBox");
	TestTrue(TEXT("Find single checkbox widget"), FoundWidgets.Num() == 1);

	FoundWidgets.Empty();
	Utility::FindChildWidgetsOfType(FoundWidgets, TestHarness.TestWidget, "SNonExistent");
	TestTrue(TEXT("Find zero nonexistent widget"), FoundWidgets.Num() == 0);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantSlateQuerierTestFindClosestWidget,
	"AI.Assistant.SlateQuerier.Utils.TypeSearch.FindClosestWidget",
	SlateQuerierTestFlags);

bool FAIAssistantSlateQuerierTestFindClosestWidget::RunTest(const FString& UnusedParameters)
{
	FSlateTestHarness TestHarness(SNew(STestWidget));

	TestHarness.GenerateWidgetPathTo(TestHarness.TestWidget->CheckBoxButton);

	TSharedPtr<SWidget> FoundWidget = Utility::FindClosestWidgetPtrOfType(
		TestHarness.WidgetPath, "SHorizontalBox");
	TestTrue(TEXT("Closest widget test finds inner widget"),
			 FoundWidget == TestHarness.TestWidget->InnerHBox);
	TestFalse(TEXT("Closest widget test does not find outer widget"),
			 FoundWidget == TestHarness.TestWidget->OuterHBox);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantSlateQuerierTestFindFirstWidget,
	"AI.Assistant.SlateQuerier.Utils.TypeSearch.FindFirstWidget",
	SlateQuerierTestFlags);

bool FAIAssistantSlateQuerierTestFindFirstWidget::RunTest(const FString& UnusedParameters)
{
	FSlateTestHarness TestHarness(SNew(STestWidget));

	TestHarness.GenerateWidgetPathTo(TestHarness.TestWidget->CheckBoxButton);

	TSharedPtr<SWidget> FoundWidget = Utility::FindFirstWidgetPtrOfType(
		TestHarness.WidgetPath, "SHorizontalBox");
	TestTrue(TEXT("First widget test finds outer widget"),
			 FoundWidget == TestHarness.TestWidget->OuterHBox);
	TestFalse(TEXT("First widget test does not find inner widget"),
			 FoundWidget == TestHarness.TestWidget->InnerHBox);

	return true;
}

#endif	// WITH_DEV_AUTOMATION_TESTS
