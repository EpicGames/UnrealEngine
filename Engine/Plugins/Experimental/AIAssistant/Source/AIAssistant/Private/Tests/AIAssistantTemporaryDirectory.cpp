// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/AIAssistantTemporaryDirectory.h"

#include "Misc/AssertionMacros.h"
#include "Containers/UnrealString.h"
#include "Misc/Guid.h"
#include "Misc/Paths.h"
#include "HAL/FileManager.h"

namespace UE::AIAssistant
{
	// Create a temporary directory.
	FTemporaryDirectory::FTemporaryDirectory() :
		// NOTE: This uses a similar path to ContentCommandlets as there doesn't seem to be functionality
		// to get the platform's temporary directory.
		Directory(
			FPaths::Combine(
				FPaths::ProjectSavedDir(), TEXT("Temp"), FGuid::NewGuid().ToString())),
		FileManager(IFileManager::Get())
	{
		verify(FileManager.MakeDirectory(*Directory, true));
	}

	// Delete a temporary directory.
	FTemporaryDirectory::~FTemporaryDirectory()
	{
		// ProjectSavedDir()/Temp will still remain, but may be empty.
		FileManager.DeleteDirectory(*Directory, /* Exists */ false, /* Tree */ true);
	}
}