// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"

class IFileManager;

namespace UE::AIAssistant
{
	// Create a temporary directory for the lifetime of this class.
	class FTemporaryDirectory
	{
	public:
		// Create a temporary directory.
		FTemporaryDirectory();

		// Disable copy.
		FTemporaryDirectory(const FTemporaryDirectory&) = delete;
		FTemporaryDirectory& operator=(const FTemporaryDirectory&) = delete;

		// Delete a temporary directory.
		~FTemporaryDirectory();

		// Get the temporary directory name.
		const FString& operator*() const { return Directory; }

	private:
		FString Directory;
		IFileManager& FileManager;
	};
}