// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace AIAssistantTest
{
	// Default flags to use for tests.
	const auto Flags =
		EAutomationTestFlags::EditorContext |
		EAutomationTestFlags::ProductFilter |
		EAutomationTestFlags::CriticalPriority;
}  // namespace AIAssistantTest

#endif  // WITH_DEV_AUTOMATION_TESTS
