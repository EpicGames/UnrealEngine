// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/UnrealString.h"
#include "UObject/Object.h"

#include "AIAssistantTestObject.generated.h"


// Simple UObject used by tests.
UCLASS()
class UAIAssistantTestObject : public UObject
{
	GENERATED_BODY()

public:
	void SetName(const FString& InstanceName)
	{
		Name = InstanceName;
	}

	const FString& GetName() const
	{
		return Name;
	}

	UPROPERTY()
	int32 TestValue = 0;

private:
	FString Name;
};