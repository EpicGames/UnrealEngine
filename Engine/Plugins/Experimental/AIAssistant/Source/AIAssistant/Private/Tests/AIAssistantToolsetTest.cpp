// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"

#include "AIAssistantToolset.h"
#include "AIAssistantTestFlags.h"

#if WITH_DEV_AUTOMATION_TESTS


BEGIN_DEFINE_SPEC(FAssistantToolsetSpec, "AI.Toolsets.Assistant", AIAssistantTest::Flags)
	FString ProjectPrompt;
	FString UserPrompt;	
END_DEFINE_SPEC(FAssistantToolsetSpec)

void FAssistantToolsetSpec::Define()
{
	BeforeEach([this]()
	{
		const UAIAssistantContextProject* ProjectSettings = GetDefault<UAIAssistantContextProject>();
		if (ProjectSettings)
		{
			ProjectPrompt = ProjectSettings->Prompt;
		}
		const UAIAssistantContextUser* UserSettings = GetDefault<UAIAssistantContextUser>();
		if (UserSettings)
		{
			UserPrompt = UserSettings->Prompt;
		}
	});

	AfterEach([this]()
	{
		UAIAssistantContextProject* ProjectSettings = GetMutableDefault<UAIAssistantContextProject>();
		if (ProjectSettings)
		{
			ProjectSettings->Prompt = ProjectPrompt;
		}
		ProjectPrompt.Empty();
		UAIAssistantContextUser* UserSettings = GetMutableDefault<UAIAssistantContextUser>();
		if (UserSettings)
		{
			UserSettings->Prompt = UserPrompt;
		}
		UserPrompt.Empty();
	});

	It("Returns engine context.", [this]()
	{
		FAIAssistantContext Context = UAIAssistantToolset::GetProjectContext();
		
		TestTrue(TEXT("Context"), Context.UnrealContext.Contains(TEXT("100 units is 1 meter")));
	});

	It("Returns project context.", [this]()
	{
		UAIAssistantContextProject* ProjectSettings = GetMutableDefault<UAIAssistantContextProject>();
		ProjectSettings->Prompt = FString(TEXT("Foo bar"));

		FAIAssistantContext Context = UAIAssistantToolset::GetProjectContext();
		TestEqual(TEXT("Context"), Context.ProjectContext, FString(TEXT("Foo bar")));
	});

	It("Returns user context.", [this]()
	{
		UAIAssistantContextUser* UserSettings = GetMutableDefault<UAIAssistantContextUser>();
		UserSettings->Prompt = FString(TEXT("Bish bosh"));

		FAIAssistantContext Context = UAIAssistantToolset::GetProjectContext();
		TestEqual(TEXT("Context"), Context.UserContext, FString(TEXT("Bish bosh")));
	});	
}


#endif  // WITH_DEV_AUTOMATION_TESTS
