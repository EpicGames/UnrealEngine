// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"

#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Editor/TransBuffer.h"

#include "AIAssistantTestFlags.h"
#include "AIAssistantTestObject.h"
#include "AIAssistantTransactionBufferManager.h"

#if WITH_DEV_AUTOMATION_TESTS

using namespace UE::AIAssistant;

BEGIN_DEFINE_SPEC(FAIAssistantTransactionBufferManagerTest,
	"AI.Assistant.TransactionBufferManager",
	AIAssistantTest::Flags)
	FString TestBufferName;
	FString TestBufferName2;
END_DEFINE_SPEC(FAIAssistantTransactionBufferManagerTest)

void FAIAssistantTransactionBufferManagerTest::Define()
{
	if (!GEditor)
	{
		AddInfo(TEXT("Test skipped: No GEditor available."));
		return;
	}

	BeforeEach([this]
		{
			TestBufferName = TEXT("TestBuffer");
			TestBufferName2 = TEXT("TestBuffer2");
		});

	AfterEach([this]
		{
			// Clean up any buffers created during tests
			FTransactionBufferManager::Shutdown();
		});

	Describe("GetTransactionBuffer", [this]
		{
			It("returns nullptr when buffer doesn't exist", [this]
				{
					TObjectPtr<UTransBuffer> Buffer =
						FTransactionBufferManager::GetTransactionBuffer(TestBufferName);
					TestNull(TEXT("Buffer should be nullptr for non-existent name"), Buffer.Get());
				});

			It("returns existing buffer after creation", [this]
				{
					FTransactionBufferManager::GetOrCreateTransactionBuffer(TestBufferName);
					TObjectPtr<UTransBuffer> Buffer =
						FTransactionBufferManager::GetTransactionBuffer(TestBufferName);
					TestNotNull(TEXT("Buffer should be returned after creation"), Buffer.Get());
				});

			It("returns the same buffer for the same name", [this]
				{
					TObjectPtr<UTransBuffer> Buffer1 =
						FTransactionBufferManager::GetOrCreateTransactionBuffer(TestBufferName);
					TObjectPtr<UTransBuffer> Buffer2 =
						FTransactionBufferManager::GetOrCreateTransactionBuffer(TestBufferName);
					TestTrue(
						TEXT("Same buffer should be returned"),
						Buffer1.Get() == Buffer2.Get());
				});
		});

	Describe("GetOrCreateTransactionBuffer", [this]
		{
			It("creates a new buffer when one doesn't exist", [this]
				{
					TObjectPtr<UTransBuffer> Buffer =
						FTransactionBufferManager::GetOrCreateTransactionBuffer(TestBufferName);
					TestNotNull(TEXT("Buffer should be created"), Buffer.Get());
				});

			It("returns the same buffer for the same name", [this]
				{
					TObjectPtr<UTransBuffer> Buffer1 =
						FTransactionBufferManager::GetOrCreateTransactionBuffer(TestBufferName);
					TObjectPtr<UTransBuffer> Buffer2 =
						FTransactionBufferManager::GetOrCreateTransactionBuffer(TestBufferName);
					TestTrue(
						TEXT("Same buffer should be returned"),
						Buffer1.Get() == Buffer2.Get());
				});

			It("returns different buffers for different names", [this]
				{
					TObjectPtr<UTransBuffer> Buffer1 =
						FTransactionBufferManager::GetOrCreateTransactionBuffer(TestBufferName);
					TObjectPtr<UTransBuffer> Buffer2 =
						FTransactionBufferManager::GetOrCreateTransactionBuffer(TestBufferName2);
					TestTrue(
						TEXT("Different buffers should be returned for different names"),
						Buffer1.Get() != Buffer2.Get());
				});
		});

	Describe("DestroyTransactionBuffer", [this]
		{
			It("returns false when buffer doesn't exist", [this]
				{
					const bool bResult = FTransactionBufferManager::DestroyTransactionBuffer(
						TEXT("NonExistentBuffer"));
					TestFalse(TEXT("Should return false for non-existent buffer"), bResult);
				});

			It("returns true when buffer is destroyed successfully", [this]
				{
					FTransactionBufferManager::GetOrCreateTransactionBuffer(TestBufferName);
					const bool bResult =
						FTransactionBufferManager::DestroyTransactionBuffer(TestBufferName);
					TestTrue(TEXT("Should return true when buffer is destroyed"), bResult);
				});

			It("removes buffer from registry after destruction", [this]
				{
					TObjectPtr<UTransBuffer> OriginalBuffer =
						FTransactionBufferManager::GetOrCreateTransactionBuffer(TestBufferName);
					FTransactionBufferManager::DestroyTransactionBuffer(TestBufferName);
					TObjectPtr<UTransBuffer> NewBuffer =
						FTransactionBufferManager::GetOrCreateTransactionBuffer(TestBufferName);
					TestTrue(
						TEXT("New buffer should be created after destruction"),
						OriginalBuffer.Get() != NewBuffer.Get());
				});

			It("cannot destroy buffer that is currently active as override", [this]
				{
					TObjectPtr<UTransBuffer> Buffer =
						FTransactionBufferManager::GetOrCreateTransactionBuffer(TestBufferName);
					FTransactionBufferManager::SetOverrideBuffer(Buffer);

					const bool bResult =
						FTransactionBufferManager::DestroyTransactionBuffer(TestBufferName);
					TestFalse(
						TEXT("Should return false when trying to destroy active override buffer"),
						bResult);

					FTransactionBufferManager::RestoreGlobalBuffer();
				});
		});

	Describe("SetOverrideBuffer", [this]
		{
			It("returns false when OverrideBuffer is null", [this]
				{
					const bool bResult = FTransactionBufferManager::SetOverrideBuffer(nullptr);
					TestFalse(TEXT("Should return false for null buffer"), bResult);
				});

			It("returns true when override is set successfully", [this]
				{
					TObjectPtr<UTransBuffer> Buffer =
						FTransactionBufferManager::GetOrCreateTransactionBuffer(TestBufferName);
					const bool bResult = FTransactionBufferManager::SetOverrideBuffer(Buffer);
					TestTrue(TEXT("Should return true when override is set"), bResult);
				});

			It("sets the override as the active editor transaction buffer", [this]
				{
					TObjectPtr<UTransBuffer> Buffer =
						FTransactionBufferManager::GetOrCreateTransactionBuffer(TestBufferName);
					FTransactionBufferManager::SetOverrideBuffer(Buffer);
					TestTrue(
						TEXT("Editor Trans should be the override buffer"),
						GEditor->Trans.Get() == Buffer.Get());
				});

			It("returns false when an override is already active", [this]
				{
					TObjectPtr<UTransBuffer> Buffer1 =
						FTransactionBufferManager::GetOrCreateTransactionBuffer(TestBufferName);
					TObjectPtr<UTransBuffer> Buffer2 =
						FTransactionBufferManager::GetOrCreateTransactionBuffer(TestBufferName2);

					const bool bFirstResult = FTransactionBufferManager::SetOverrideBuffer(Buffer1);
					TestTrue(TEXT("First override should succeed"), bFirstResult);

					const bool bSecondResult = FTransactionBufferManager::SetOverrideBuffer(Buffer2);
					TestFalse(
						TEXT("Second override should fail while first is active"),
						bSecondResult);

					// Verify the first override is still active
					TestTrue(
						TEXT("First override buffer should still be active"),
						GEditor->Trans.Get() == Buffer1.Get());
				});
		});

	Describe("RestoreGlobalBuffer", [this]
		{
			It("restores the original global buffer after override", [this]
				{
					UTransactor* OriginalTrans = GEditor->Trans.Get();
					TObjectPtr<UTransBuffer> Buffer =
						FTransactionBufferManager::GetOrCreateTransactionBuffer(TestBufferName);
					FTransactionBufferManager::SetOverrideBuffer(Buffer);
					FTransactionBufferManager::RestoreGlobalBuffer();
					TestTrue(
						TEXT("Editor Trans should be restored to original"),
						GEditor->Trans.Get() == OriginalTrans);
				});

			It("clears the override state after restoration", [this]
				{
					TObjectPtr<UTransBuffer> Buffer =
						FTransactionBufferManager::GetOrCreateTransactionBuffer(TestBufferName);
					FTransactionBufferManager::SetOverrideBuffer(Buffer);
					FTransactionBufferManager::RestoreGlobalBuffer();
					TestFalse(
						TEXT("Override should not be active after restoration"),
						FTransactionBufferManager::IsOverrideActive());
				});
		});

	Describe("IsOverrideActive", [this]
		{
			It("returns false when no override is set", [this]
				{
					TestFalse(
						TEXT("Should return false when no override is active"),
						FTransactionBufferManager::IsOverrideActive());
				});

			It("returns true when override is set", [this]
				{
					TObjectPtr<UTransBuffer> Buffer =
						FTransactionBufferManager::GetOrCreateTransactionBuffer(TestBufferName);
					FTransactionBufferManager::SetOverrideBuffer(Buffer);
					TestTrue(
						TEXT("Should return true when override is active"),
						FTransactionBufferManager::IsOverrideActive());
				});

			It("returns false after override is restored", [this]
				{
					TObjectPtr<UTransBuffer> Buffer =
						FTransactionBufferManager::GetOrCreateTransactionBuffer(TestBufferName);
					FTransactionBufferManager::SetOverrideBuffer(Buffer);
					FTransactionBufferManager::RestoreGlobalBuffer();
					TestFalse(
						TEXT("Should return false after override is restored"),
						FTransactionBufferManager::IsOverrideActive());
				});
		});

	Describe("Shutdown", [this]
		{
			It("clears all registered buffers", [this]
				{
					FTransactionBufferManager::GetOrCreateTransactionBuffer(TestBufferName);
					FTransactionBufferManager::GetOrCreateTransactionBuffer(TestBufferName2);
					FTransactionBufferManager::Shutdown();

					// After shutdown, getting a buffer should return nullptr
					TObjectPtr<UTransBuffer> Buffer =
						FTransactionBufferManager::GetTransactionBuffer(TestBufferName);
					TestNull(
						TEXT("Buffer should be cleared after shutdown"),
						Buffer.Get());
				});

			It("restores global buffer if override was active", [this]
				{
					UTransactor* OriginalTrans = GEditor->Trans.Get();
					TObjectPtr<UTransBuffer> Buffer =
						FTransactionBufferManager::GetOrCreateTransactionBuffer(TestBufferName);
					FTransactionBufferManager::SetOverrideBuffer(Buffer);
					FTransactionBufferManager::Shutdown();

					TestTrue(
						TEXT("Global buffer should be restored after shutdown"),
						GEditor->Trans.Get() == OriginalTrans);
					TestFalse(
						TEXT("Override should not be active after shutdown"),
						FTransactionBufferManager::IsOverrideActive());
				});
		});

	Describe("GetFilenamesFromUndoStack", [this]
		{
			It("returns empty set when UndoStack is nullptr", [this]
				{
					TSet<FString> Filenames =
						FTransactionBufferManager::GetFilenamesFromUndoStack(nullptr);
					TestEqual(
						TEXT("Should return empty set for nullptr"),
						Filenames.Num(), 0);
				});

			It("returns empty set when UndoStack has no transactions", [this]
				{
					TObjectPtr<UTransBuffer> Buffer =
						FTransactionBufferManager::GetOrCreateTransactionBuffer(TestBufferName);
					TSet<FString> Filenames =
						FTransactionBufferManager::GetFilenamesFromUndoStack(Buffer);
					TestEqual(
						TEXT("Should return empty set for empty buffer"),
						Filenames.Num(), 0);
				});

			It("returns filenames from active transactions", [this]
				{
					TObjectPtr<UTransBuffer> Buffer =
						FTransactionBufferManager::GetOrCreateTransactionBuffer(TestBufferName);

					// Create a test object and record a transaction
					UPackage* TestPackage = NewObject<UPackage>(
						nullptr, TEXT("/Temp/AIAssistantTestPackage"), RF_Transient);
					UAIAssistantTestObject* TestObject = NewObject<UAIAssistantTestObject>(
						TestPackage, TEXT("TestObject"), RF_Transactional);

					// Begin a transaction on the buffer
					Buffer->Begin(TEXT("Test Transaction"), FText::GetEmpty());
					TestObject->Modify();
					TestObject->TestValue = 1;
					Buffer->End();

					TSet<FString> Filenames =
						FTransactionBufferManager::GetFilenamesFromUndoStack(Buffer);

					TestTrue(
						TEXT("Should contain the test package path"),
						Filenames.Contains(TEXT("/Temp/AIAssistantTestPackage")));
				});

			It("does not return filenames from undone transactions", [this]
				{
					TObjectPtr<UTransBuffer> Buffer =
						FTransactionBufferManager::GetOrCreateTransactionBuffer(TestBufferName);

					// Create a test object and record a transaction
					UPackage* TestPackage = NewObject<UPackage>(
						nullptr, TEXT("/Temp/AIAssistantUndoTestPackage"), RF_Transient);
					UAIAssistantTestObject* TestObject = NewObject<UAIAssistantTestObject>(
						TestPackage, TEXT("TestObject"), RF_Transactional);

					// Record a transaction
					Buffer->Begin(TEXT("Test Transaction"), FText::GetEmpty());
					TestObject->Modify();
					TestObject->TestValue = 1;
					Buffer->End();

					// Undo the transaction
					Buffer->Undo();

					TSet<FString> Filenames =
						FTransactionBufferManager::GetFilenamesFromUndoStack(Buffer);

					TestFalse(
						TEXT("Should not contain the undone package path"),
						Filenames.Contains(TEXT("/Temp/AIAssistantUndoTestPackage")));
				});
		});
}

#endif  // WITH_DEV_AUTOMATION_TESTS