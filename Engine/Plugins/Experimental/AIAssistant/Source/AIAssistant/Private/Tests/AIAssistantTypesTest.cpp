// Copyright Epic Games, Inc. All Rights Reserved.

#include "Containers/UnrealString.h"
#include "Misc/AutomationTest.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"

#include "AIAssistantMessageUtils.h"
#include "AIAssistantTestFlags.h"
#include "AIAssistantTypes.h"

#if WITH_DEV_AUTOMATION_TESTS

using namespace UE::AIAssistant;

BEGIN_DEFINE_SPEC(AIAssistantTypesToolCallContent, "AI.Assistant.Types.ToolCallContent",
	AIAssistantTest::Flags)
	FToolCallContent ToolCall;
	FToolCallContent WithIdAndResponse;
END_DEFINE_SPEC(AIAssistantTypesToolCallContent)

void AIAssistantTypesToolCallContent::Define()
{
	BeforeEach([this]
		{
			ToolCall.Name = TEXT("agent_status");
			ToolCall.ArgumentsRawJson = TEXT(R"json({"foo": 1})json");
			ToolCall.ToolCallId.Reset();
			ToolCall.ResponseRequired.Reset();

			WithIdAndResponse.Name = TEXT("agent_status");
			WithIdAndResponse.ArgumentsRawJson = TEXT(R"json({"foo": 1})json");
			WithIdAndResponse.ToolCallId = TEXT("ToolCall1-0");
			WithIdAndResponse.ResponseRequired = true;
		});

	Describe("JSON Serialization", [this]
	{
		It("parses arguments as a JSON object and stores it as a raw json string", [this]
		{
			const FString Input = TEXT(R"json(
			{
				"name": "agent_status",
				"arguments": {"foo":1,"bar":true,"nested":{"x":"y"}}
			})json");
			FToolCallContent Content;
			const bool bParsedOk = Content.FromJson(Input);
			TestTrue(TEXT("FromJson should succeed"), bParsedOk);
			TestEqual(TEXT("Name parsed"), Content.Name, FString(TEXT("agent_status")));
			TestEqual(
				TEXT("ArgumentsRawJson preserved as-is"),
				Content.ArgumentsRawJson,
				TEXT(R"json({"foo":1,"bar":true,"nested":{"x":"y"}})json"));
		});

		It("emits arguments as a raw json string", [this]
		{
			// Now serialize it back and verify the JSON type is Object, not String.
			const FString Json = ToolCall.ToJson();
			TSharedPtr<FJsonObject> Object;
			const TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(Json);
			if (!TestTrue(
				TEXT("Serialized output should be valid JSON object"),
				FJsonSerializer::Deserialize(Reader, Object) && Object.IsValid()))
			{
				return;
			}
			const TSharedPtr<FJsonValue> Arguments = Object->TryGetField(TEXT("arguments"));
			if (!TestTrue(TEXT("'arguments' field should exist"), Arguments.IsValid()))
			{
				return;
			}
			TestEqual(
				TEXT("'arguments' should be a JSON object (not a string)"),
				Arguments->Type, EJson::Object);
			const TSharedPtr<FJsonObject> ArgumentsObject = Arguments->AsObject();
			if (!TestTrue(TEXT("'arguments' object should be valid"), ArgumentsObject.IsValid()))
			{
				return;
			}
			TestEqual(TEXT("foo == 1"), ArgumentsObject->GetIntegerField(TEXT("foo")), 1);
		});
	});
}


BEGIN_DEFINE_SPEC(AIAssistantTypesToolResponseContent, "AI.Assistant.Types.ToolResponseContent",
	AIAssistantTest::Flags)
	FToolResponseContent Response;
END_DEFINE_SPEC(AIAssistantTypesToolResponseContent)

void AIAssistantTypesToolResponseContent::Define()
{
	BeforeEach([this]
		{
			Response.ToolCallId = TEXT("ToolCall1-0");
			Response.Name = TEXT("agent_status");
			Response.ResponseRawJson = TEXT(R"json({"ok": true})json");
			Response.Success.Reset();
			Response.ErrorMessage.Reset();
		});

	Describe("JSON Serialization", [this]
	{
		It("parses response as a JSON object and stores it as a raw json string", [this]
		{
			const FString Input = TEXT(R"json(
			{
				"toolCallId": "ToolCall1-0",
				"name": "agent_status",
				"response": {"ok":true}
			})json");

			FToolResponseContent Content;
			const bool bParsedOk = Content.FromJson(Input);
			TestTrue(TEXT("FromJson should succeed"), bParsedOk);

			TestEqual(TEXT("ToolCallId parsed"), Content.ToolCallId, FString(TEXT("ToolCall1-0")));
			TestEqual(TEXT("Name parsed"), Content.Name, FString(TEXT("agent_status")));

			// Raw field should be stored as JSON text (object), not a quoted JSON string.
			TestEqual(
				TEXT("ResponseRawJson preserved as-is"),
				Content.ResponseRawJson,
				TEXT(R"json({"ok":true})json"));
		});

		It("emits response as a raw json string", [this]
		{
			// Now serialize it back and verify the JSON type is Object, not String.
			const FString Output = Response.ToJson();
			TSharedPtr<FJsonObject> Object;
			const TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(Output);
			if (!TestTrue(
				TEXT("Serialized output should be valid JSON object"),
				FJsonSerializer::Deserialize(Reader, Object) && Object.IsValid()))
			{
				return;
			}
			const TSharedPtr<FJsonValue> ResponseField = Object->TryGetField(TEXT("response"));
			if (!TestTrue(TEXT("'response' field should exist"), ResponseField.IsValid()))
			{
				return;
			}
			TestEqual(TEXT("'response' should be a JSON object (not a string)"),
				ResponseField->Type, EJson::Object);
			const TSharedPtr<FJsonObject> ResponseObject = ResponseField->AsObject();
			if (!TestTrue(TEXT("'response' object should be valid"), ResponseObject.IsValid()))
			{
				return;
			}
			TestTrue(TEXT("ok == true"), ResponseObject->GetBoolField(TEXT("ok")));
		});
	});
}

BEGIN_DEFINE_SPEC(AIAssistantTypesMessageId, "AI.Assistant.Types.MessageId",
	AIAssistantTest::Flags)
	FMessageId MessageId;
END_DEFINE_SPEC(AIAssistantTypesMessageId)

void AIAssistantTypesMessageId::Define()
{
	BeforeEach([this]
		{
			MessageId = MakeMessageId(1);
		});

	Describe("Equality operator", [this]
		{
			It("returns true when the object is identical", [this]
				{
					FMessageId MessageIdSame = MakeMessageId(1);
					TestTrue(TEXT("MessageIds created with same index should be equal"),
						MessageId == MessageIdSame);
				});
			It("returns false when the Id is different", [this]
				{
					FMessageId MessageIdDifferentIndex = MakeMessageId(2);
					TestFalse(TEXT("MessageIds with different Id should not be equal"),
						MessageId == MessageIdDifferentIndex);
				});
			It("returns false when the Type is different", [this]
				{
					FMessageId MessageIdDifferentType = MakeMessageId(1);
					MessageIdDifferentType.Type = TEXT("OtherType");
					TestFalse(TEXT("MessageIds with different Type should not be equal"),
						MessageId == MessageIdDifferentType);
				});
		});

	Describe("hash function", [this]
		{
			It("produces same hash for same MessageId", [this]
				{
					FMessageId MessageIdSame = MakeMessageId(1);

					const uint32 Hash = GetTypeHash(MessageId);
					const uint32 HashSame = GetTypeHash(MessageIdSame);

					TestEqual(TEXT("Same MessageId value should produce same hash"), Hash, HashSame);
				});
			It("produces different hash for different MessageId", [this]
				{
					FMessageId MessageIdDifferent = MakeMessageId(2);

					const uint32 Hash = GetTypeHash(MessageId);
					const uint32 HashDifferent = GetTypeHash(MessageIdDifferent);

					TestNotEqual(TEXT("Different MessageId value should produce different hash"),
						Hash, HashDifferent);
				});
		});
}

BEGIN_DEFINE_SPEC(AIAssistantTypesConversationId, "AI.Assistant.Types.ConversationId",
	AIAssistantTest::Flags)
	FConversationId ConversationId;
END_DEFINE_SPEC(AIAssistantTypesConversationId)

void AIAssistantTypesConversationId::Define()
{
	BeforeEach([this]
		{
			ConversationId = MakeConversationId(1);
		});

	Describe("Equality operator", [this]
		{
			It("returns true when the object is identical", [this]
				{
					FConversationId ConversationIdSame = MakeConversationId(1);
					TestTrue(TEXT("ConversationIds with same Id should be equal"),
						ConversationId == ConversationIdSame);
				});

			It("returns false when the Id is different", [this]
				{
					FConversationId ConversationIdDifferent = MakeConversationId(2);
					TestFalse(TEXT("ConversationIds with different Id should not be equal"),
						ConversationId == ConversationIdDifferent);
				});
		});

	Describe("hash function", [this]
		{
			It("produces same hash for same ConversationId", [this]
				{
					FConversationId ConversationIdSame = MakeConversationId(1);

					const uint32 Hash = GetTypeHash(ConversationId);
					const uint32 HashSame = GetTypeHash(ConversationIdSame);

					TestEqual(TEXT("Same ConversationId value should produce same hash"), Hash, HashSame);
				});

			It("produces different hash for different ConversationId", [this]
				{
					FConversationId ConversationIdDifferent = MakeConversationId(2);

					const uint32 Hash = GetTypeHash(ConversationId);
					const uint32 HashDifferent = GetTypeHash(ConversationIdDifferent);

					TestNotEqual(TEXT("Different ConversationId value should produce different hash"),
						Hash, HashDifferent);
				});
		});
}

BEGIN_DEFINE_SPEC(AIAssistantTypesMessage, "AI.Assistant.Types.Message",
	AIAssistantTest::Flags)
END_DEFINE_SPEC(AIAssistantTypesMessage)

void AIAssistantTypesMessage::Define()
{
	Describe(TEXT("messageRole deserialization"), [this]
	{
		It(TEXT("maps 'unknown' to EMessageRole::Unknown"), [this]
		{
			FMessage Message;
			const bool bOk = Message.FromJson(
				TEXT(R"json({"messageRole":"unknown","messageContent":[]})json"));
			(void)TestTrue(TEXT("FromJson succeeds"), bOk);
			(void)TestEqual(TEXT("MessageRole"), Message.MessageRole, EMessageRole::Unknown);
		});

		It(TEXT("maps an unrecognized role string to EMessageRole::Unknown"), [this]
		{
			FMessage Message;
			const bool bOk = Message.FromJson(
				TEXT(R"json({"messageRole":"system","messageContent":[]})json"));
			(void)TestTrue(TEXT("FromJson succeeds"), bOk);
			(void)TestEqual(TEXT("MessageRole"), Message.MessageRole, EMessageRole::Unknown);
		});
	});

	Describe(TEXT("messageContent contentType deserialization"), [this]
	{
		It(TEXT("maps 'unknown' contentType to EMessageContentType::Unknown"), [this]
		{
			FMessage Message;
			const TCHAR* Json = TEXT(
				R"json({"messageRole":"user","messageContent":[{"contentType":"unknown","content":{}}]})json");
			const bool bOk = Message.FromJson(Json);
			(void)TestTrue(TEXT("FromJson succeeds"), bOk);
			if (!TestEqual(TEXT("ContentCount"), Message.MessageContent.Num(), 1))
			{
				return;
			}
			(void)TestEqual(TEXT("ContentType"),
				Message.MessageContent[0].ContentType, EMessageContentType::Unknown);
			(void)TestTrue(TEXT("HoldsUnknownContent"),
				Message.MessageContent[0].Content.IsType<FUnknownMessageContent>());
		});

		It(TEXT("maps an unrecognized contentType string to EMessageContentType::Unknown"), [this]
		{
			FMessage Message;
			const TCHAR* Json = TEXT(
				R"json({"messageRole":"user","messageContent":[{"contentType":"image","content":{}}]})json");
			const bool bOk = Message.FromJson(Json);
			(void)TestTrue(TEXT("FromJson succeeds"), bOk);
			if (!TestEqual(TEXT("ContentCount"), Message.MessageContent.Num(), 1))
			{
				return;
			}
			(void)TestEqual(TEXT("ContentType"),
				Message.MessageContent[0].ContentType, EMessageContentType::Unknown);
			(void)TestTrue(TEXT("HoldsUnknownContent"),
				Message.MessageContent[0].Content.IsType<FUnknownMessageContent>());
		});
	});
}

BEGIN_DEFINE_SPEC(AIAssistantTypesPendingFileMetadata, "AI.Assistant.Types.PendingFileMetadata",
	AIAssistantTest::Flags)
END_DEFINE_SPEC(AIAssistantTypesPendingFileMetadata)

void AIAssistantTypesPendingFileMetadata::Define()
{
	Describe("JSON Serialization", [this]
	{
		It("serializes all fields correctly", [this]
		{
			FPendingFileMetadata Metadata;
			Metadata.DisplayName = TEXT("main.cpp");
			Metadata.FullPath = TEXT("/src/main.cpp");
			Metadata.Status = EPendingFileStatus::Modified;

			const FString Json = Metadata.ToJson(false);
			TSharedPtr<FJsonObject> Object;
			const TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(Json);
			if (!TestTrue(TEXT("Should parse as valid JSON"),
				FJsonSerializer::Deserialize(Reader, Object) && Object.IsValid()))
			{
				return;
			}

			FString DisplayName, FullPath, Status;
			if (!TestTrue(TEXT("displayName exists"),
					Object->TryGetStringField(TEXT("displayName"), DisplayName)) ||
				!TestTrue(TEXT("fullPath exists"),
					Object->TryGetStringField(TEXT("fullPath"), FullPath)) ||
				!TestTrue(TEXT("status exists"),
					Object->TryGetStringField(TEXT("status"), Status)))
			{
				return;
			}
			TestEqual(TEXT("displayName"), DisplayName, TEXT("main.cpp"));
			TestEqual(TEXT("fullPath"), FullPath, TEXT("/src/main.cpp"));
			TestEqual(TEXT("status"), Status, TEXT("modified"));
		});

		It("serializes optional originalPath when set", [this]
		{
			FPendingFileMetadata Metadata;
			Metadata.DisplayName = TEXT("config.json");
			Metadata.FullPath = TEXT("/new/config.json");
			Metadata.Status = EPendingFileStatus::Moved;
			Metadata.OriginalPath = TEXT("/old/config.json");

			const FString Json = Metadata.ToJson(false);
			TSharedPtr<FJsonObject> Object;
			const TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(Json);
			if (!TestTrue(TEXT("Should parse as valid JSON"),
				FJsonSerializer::Deserialize(Reader, Object) && Object.IsValid()))
			{
				return;
			}

			TestEqual(TEXT("status"), Object->GetStringField(TEXT("status")), TEXT("moved"));
			FString OriginalPath;
			if (!TestTrue(TEXT("originalPath exists"),
					Object->TryGetStringField(TEXT("originalPath"), OriginalPath)))
			{
				return;
			}
			TestEqual(TEXT("originalPath"), OriginalPath, TEXT("/old/config.json"));
		});

		It("parses all status values correctly", [this]
		{
			auto TestStatus = [this](const TCHAR* StatusStr, EPendingFileStatus ExpectedStatus)
			{
				const FString Input = FString::Printf(
					TEXT(R"json({"displayName":"test","fullPath":"/test","status":"%s"})json"),
					StatusStr);
				FPendingFileMetadata Metadata;
				if (!TestTrue(FString::Printf(TEXT("FromJson should succeed for %s"), StatusStr),
					Metadata.FromJson(Input)))
				{
					return;
				}
				TestEqual(FString::Printf(TEXT("Status should be %s"), StatusStr),
					Metadata.Status, ExpectedStatus);
			};

			TestStatus(TEXT("added"), EPendingFileStatus::Added);
			TestStatus(TEXT("removed"), EPendingFileStatus::Removed);
			TestStatus(TEXT("modified"), EPendingFileStatus::Modified);
			TestStatus(TEXT("moved"), EPendingFileStatus::Moved);
		});
	});
}

BEGIN_DEFINE_SPEC(AIAssistantTypesUpdatePendingFileListOptions,
	"AI.Assistant.Types.UpdatePendingFileListOptions", AIAssistantTest::Flags)
END_DEFINE_SPEC(AIAssistantTypesUpdatePendingFileListOptions)

void AIAssistantTypesUpdatePendingFileListOptions::Define()
{
	Describe("JSON Serialization", [this]
	{
		It("serializes with conversationId when set", [this]
		{
			FUpdatePendingFileListOptions Options;
			Options.ConversationId.Emplace();
			Options.ConversationId->Id = TEXT("conv-123");
			Options.ConversationId->Type = TEXT("ConversationId");

			FPendingFileMetadata& File = Options.Files.AddDefaulted_GetRef();
			File.DisplayName = TEXT("test.cpp");
			File.FullPath = TEXT("/src/test.cpp");
			File.Status = EPendingFileStatus::Added;

			const FString Json = Options.ToJson(false);
			TSharedPtr<FJsonObject> Object;
			const TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(Json);
			if (!TestTrue(TEXT("Should parse as valid JSON"),
				FJsonSerializer::Deserialize(Reader, Object) && Object.IsValid()))
			{
				return;
			}

			const TSharedPtr<FJsonObject>* ConvIdObj;
			if (!TestTrue(TEXT("conversationId exists"),
					Object->TryGetObjectField(TEXT("conversationId"), ConvIdObj)))
			{
				return;
			}
			TestEqual(TEXT("conversationId.id"), (*ConvIdObj)->GetStringField(TEXT("id")), TEXT("conv-123"));

			const TArray<TSharedPtr<FJsonValue>>* FilesArray;
			if (!TestTrue(TEXT("files array exists"), Object->TryGetArrayField(TEXT("files"), FilesArray)))
			{
				return;
			}
			TestEqual(TEXT("files count"), FilesArray->Num(), 1);
		});

		It("serializes without conversationId when not set", [this]
		{
			FUpdatePendingFileListOptions Options;
			// ConversationId not set

			FPendingFileMetadata& File = Options.Files.AddDefaulted_GetRef();
			File.DisplayName = TEXT("test.cpp");
			File.FullPath = TEXT("/src/test.cpp");
			File.Status = EPendingFileStatus::Removed;

			const FString Json = Options.ToJson(false);
			TSharedPtr<FJsonObject> Object;
			const TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(Json);
			if (!TestTrue(TEXT("Should parse as valid JSON"),
				FJsonSerializer::Deserialize(Reader, Object) && Object.IsValid()))
			{
				return;
			}

			TestFalse(TEXT("conversationId should not exist"), Object->HasField(TEXT("conversationId")));

			const TArray<TSharedPtr<FJsonValue>>* FilesArray;
			if (!TestTrue(TEXT("files array exists"), Object->TryGetArrayField(TEXT("files"), FilesArray)))
			{
				return;
			}
			TestEqual(TEXT("files count"), FilesArray->Num(), 1);
		});
	});
}

#endif  // WITH_DEV_AUTOMATION_TESTS
