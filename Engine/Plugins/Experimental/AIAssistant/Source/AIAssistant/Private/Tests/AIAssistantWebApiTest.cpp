// Copyright Epic Games, Inc. All Rights Reserved.

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "Misc/AutomationTest.h"
#include "Templates/Tuple.h"

#include "AIAssistantFakeWebApi.h"
#include "AIAssistantFakeWebJavaScriptExecutor.h"
#include "AIAssistantFakeWebJavaScriptDelegateBinder.h"
#include "AIAssistantMessageUtils.h"
#include "AIAssistantTestFlags.h"
#include "AIAssistantWebApi.h"
#include "AIAssistantWebJavaScriptResultDelegateAccessor.h"

#if WITH_DEV_AUTOMATION_TESTS

using namespace UE::AIAssistant;

namespace UE::AIAssistant
{
	struct FFakeWebApiContainer
	{
		FFakeWebApiContainer() :
			WebApi(WebJavaScriptExecutor, WebJavaScriptDelegateBinder, UnbindDelegate) {}

		FFakeWebJavaScriptExecutor WebJavaScriptExecutor;
		FFakeWebJavaScriptDelegateBinder WebJavaScriptDelegateBinder;
		FSimpleMulticastDelegate UnbindDelegate;
		FFakeWebApi WebApi;

		FFakeWebApi& operator*() { return WebApi; }
		FFakeWebApi* operator->() { return &WebApi; }
	};

	class FWebApiAccessor
	{
	public:
		static FString FormatFunctionCall(
			FWebApi& WebApi, const TCHAR* InstanceName,
			const TCHAR* FunctionName, const TCHAR* Arguments = TEXT(""),
			const FString& HandlerId = FString())
		{
			return WebApi.FormatFunctionCall(InstanceName, FunctionName, Arguments, HandlerId);
		}

		static FString FormatFunctionCallback(
			FWebApi& WebApi, const FString& HandlerId) {
			return WebApi.FormatFunctionCallback(HandlerId);
		}

		static TPair<FString, FString> FormatResultAndErrorHandlers(
			FWebApi& WebApi, const FString& HandlerId)
		{
			return WebApi.FormatResultAndErrorHandlers(HandlerId);
		}

		static UAIAssistantWebJavaScriptResultDelegate& GetJavaScriptResultDelegate(
			FWebApi& WebApi)
		{
			return *WebApi.WebJavaScriptResultDelegate;
		}

		static const FString& GetConvertErrorToString()
		{
			return FWebApi::ConvertErrorToString;
		}

		static const FString& GetWebApiAvailableFunction()
		{
			return FWebApi::WebApiAvailableFunction;
		}
	};
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantWebApiTestFormatFunctionCallGlobalNoArgs,
	"AI.Assistant.WebApi.FormatFunctionCallGlobalNoArgs",
	AIAssistantTest::Flags);

bool FAIAssistantWebApiTestFormatFunctionCallGlobalNoArgs::RunTest(const FString& UnusedParameters)
{
	FFakeWebApiContainer WebApi;
	return TestEqual(
		TEXT("FormatFunctionCallGlobalNoArgs"),
		FWebApiAccessor::FormatFunctionCall(*WebApi, nullptr, TEXT("test"), TEXT("")),
		TEXT(R"js(
try {
  Promise.resolve(test()).then(
    (result) => {
      
    },
    (error) => {
      
    });
} catch (error) {
  
}
)js"));
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantWebApiTestFormatFunctionCallNoArgs,
	"AI.Assistant.WebApi.FormatFunctionCallNoArgs",
	AIAssistantTest::Flags);

bool FAIAssistantWebApiTestFormatFunctionCallNoArgs::RunTest(const FString& UnusedParameters)
{
	FFakeWebApiContainer WebApi;
	return TestEqual(
		TEXT("FormatFunctionCallNoArgs"),
		FWebApiAccessor::FormatFunctionCall(*WebApi, TEXT("window.eda"), TEXT("test"), TEXT("")),
		TEXT(R"js(
try {
  Promise.resolve(window.eda.test()).then(
    (result) => {
      
    },
    (error) => {
      
    });
} catch (error) {
  
}
)js"));
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantWebApiTestFormatFunctionCallWithArgs,
	"AI.Assistant.WebApi.FormatFunctionCallWithArgs",
	AIAssistantTest::Flags);

bool FAIAssistantWebApiTestFormatFunctionCallWithArgs::RunTest(const FString& UnusedParameters)
{
	FFakeWebApiContainer WebApi;
	return TestEqual(
		TEXT("FormatFunctionCallWithArgs"),
		FWebApiAccessor::FormatFunctionCall(
			*WebApi, TEXT("window.eda"), TEXT("test"), TEXT("{foo: 'bar'}")),
		TEXT(R"js(
try {
  Promise.resolve(window.eda.test({foo: 'bar'})).then(
    (result) => {
      
    },
    (error) => {
      
    });
} catch (error) {
  
}
)js"));
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantWebApiTestFormatResultAndErrorHandlers,
	"AI.Assistant.WebApi.FormatResultAndErrorHandlers",
	AIAssistantTest::Flags);

bool FAIAssistantWebApiTestFormatResultAndErrorHandlers::RunTest(
	const FString& UnusedParameters)
{
	FFakeWebApiContainer WebApi;
	const FString FakeHandlerId = TEXT("foobar");
	const auto Handlers = FWebApiAccessor::FormatResultAndErrorHandlers(*WebApi, FakeHandlerId);
	const auto& JavaScriptResultDelegate = FWebApiAccessor::GetJavaScriptResultDelegate(*WebApi);
	(void)TestEqual(
		TEXT("ResultHandler"),
		Handlers.Key,
		FString::Format(
			*JavaScriptResultDelegate.FormatJavaScriptHandler(FakeHandlerId),
			{ TEXT("result"), TEXT("false") }));
	(void)TestEqual(
		TEXT("ErrorHandler"),
		Handlers.Value,
		FString::Format(
			*JavaScriptResultDelegate.FormatJavaScriptHandler(FakeHandlerId),
			{ *FWebApiAccessor::GetConvertErrorToString(), TEXT("true")}));
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantWebApiTestFormatFunctionCallWithResultHandler,
	"AI.Assistant.WebApi.FormatFunctionCallWithResultHandler",
	AIAssistantTest::Flags);

bool FAIAssistantWebApiTestFormatFunctionCallWithResultHandler::RunTest(
	const FString& UnusedParameters)
{
	FFakeWebApiContainer WebApi;
	const FString FakeHandlerId = TEXT("foobar");
	const auto Handlers = FWebApiAccessor::FormatResultAndErrorHandlers(*WebApi, FakeHandlerId);
	return TestEqual(
		TEXT("FormatFunctionCallWithResultHandler"),
		FWebApiAccessor::FormatFunctionCall(
			*WebApi, TEXT("window.eda"), TEXT("test"), TEXT(""), FakeHandlerId),
		FString::Format(
			TEXT(R"js(
try {
  Promise.resolve(window.eda.test()).then(
    (result) => {
      {HandleResult}
    },
    (error) => {
      {HandleError}
    });
} catch (error) {
  {HandleError}
}
)js"),
			{
				{ TEXT("HandleResult"), *Handlers.Key },
				{ TEXT("HandleError"), *Handlers.Value },
			}));
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantWebApiTestFormatFunctionCallback,
	"AI.Assistant.WebApi.FormatFunctionCallback",
	AIAssistantTest::Flags);

bool FAIAssistantWebApiTestFormatFunctionCallback::RunTest(const FString& UnusedParameters)
{
	FFakeWebApiContainer WebApi;
	return TestEqual(
		TEXT("FormatFunctionCallback"),
		FWebApiAccessor::FormatFunctionCallback(*WebApi, TEXT("")),
		TEXT(R"js(
(result) => {
  
}
)js"));
}


IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantWebApiTestFormatFunctionCallbackWithResultHandler,
	"AI.Assistant.WebApi.FormatFunctionCallbackWithResultHandler",
	AIAssistantTest::Flags);

bool FAIAssistantWebApiTestFormatFunctionCallbackWithResultHandler::RunTest(
	const FString& UnusedParameters)
{
	FFakeWebApiContainer WebApi;
	const FString FakeHandlerId = TEXT("foobar");
	const auto Handlers = FWebApiAccessor::FormatResultAndErrorHandlers(*WebApi, FakeHandlerId);
	return TestEqual(
		TEXT("FormatFunctionCallWithResultHandler"),
		FWebApiAccessor::FormatFunctionCallback(*WebApi, FakeHandlerId),
		FString::Format(
			TEXT(R"js(
(result) => {
  {HandleResult}
}
)js"),
			{
				{ TEXT("HandleResult"), *Handlers.Key },
				{ TEXT("HandleError"), *Handlers.Value },
			}));
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantWebApiTestIsAvailable,
	"AI.Assistant.WebApi.IsAvailable",
	AIAssistantTest::Flags);

bool FAIAssistantWebApiTestIsAvailable::RunTest(const FString& UnusedParameters)
{
	FFakeWebApiContainer WebApi;
	auto Result = WebApi->IsAvailable();
	FWebApiBoolResult ExpectedResult;
	ExpectedResult.bValue = true;
	return WebApi->TestExpectAsyncFunctionCallAndComplete(
		*this, TEXT(""),
		*FWebApiAccessor::GetWebApiAvailableFunction(),
		TEXT(""), Result, *ExpectedResult.ToJson(false), false);
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantWebApiTestCreateConversation,
	"AI.Assistant.WebApi.CreateConversation",
	AIAssistantTest::Flags);

bool FAIAssistantWebApiTestCreateConversation::RunTest(const FString& UnusedParameters)
{
	FFakeWebApiContainer WebApi;
	auto Result = WebApi->CreateConversation();
	return WebApi->TestExpectAsyncFunctionCallAndComplete(
		*this, nullptr, TEXT("createConversation"), TEXT(""), Result, TEXT(""), false);
}

BEGIN_DEFINE_SPEC(FAIAssistantWebApiTest, "AI.Assistant.WebApi", AIAssistantTest::Flags);
	FConversation Conversation;
END_DEFINE_SPEC(FAIAssistantWebApiTest)

void FAIAssistantWebApiTest::Define()
{
	Describe("GetConversation", [this]
		{
			It("passes the ConversationId to getConversation when supplied", [this]
				{
					FFakeWebApiContainer WebApi;
					TOptional<FConversationId> ConversationId = MakeConversationId(123);
					auto Result = WebApi->GetConversation(ConversationId);
					return WebApi->TestExpectAsyncFunctionCallAndComplete(
						*this, nullptr, TEXT("getConversation"),
						TEXT(R"json({"id":"Conversation123","type":"ConversationId"})json"),
						Result, *Conversation.ToJson(false), false);
				});
			It("passes null to getConversation when the optional is a nullopt", [this]
				{
					FFakeWebApiContainer WebApi;
					TOptional<FConversationId> ConversationId;
					auto Result = WebApi->GetConversation(ConversationId);
					return WebApi->TestExpectAsyncFunctionCallAndComplete(
						*this, nullptr, TEXT("getConversation"),
						TEXT(R"json(null)json"), Result, *Conversation.ToJson(false), false);
				});
		});

	Describe(TEXT("UnbindDelegate"), [this]
		{
			It(TEXT("Should cancel callbacks on broadcast"), [this]
				{
					FFakeWebApiContainer WebApi;
					auto Result = WebApi->CreateConversation();
					WebApi.UnbindDelegate.Broadcast();
					auto NothingOrError = Result.Consume();
					if (!TestTrue(TEXT("HasError"), NothingOrError.HasError())) return;
					(void)TestEqual(
						TEXT("Canceled"), NothingOrError.GetError(),
						UAIAssistantWebJavaScriptResultDelegate::CanceledError);
				});
		}
	);

	Describe("UpdatePendingFileList", [this]
	{
		It("calls updatePendingFileList with options", [this]
		{
			FFakeWebApiContainer WebApi;
			FUpdatePendingFileListOptions Options;
			Options.ConversationId.Emplace();
			Options.ConversationId->Id = TEXT("conv-1");
			Options.ConversationId->Type = TEXT("ConversationId");

			FPendingFileMetadata& File = Options.Files.AddDefaulted_GetRef();
			File.DisplayName = TEXT("main.cpp");
			File.FullPath = TEXT("/src/main.cpp");
			File.Status = EPendingFileStatus::Modified;

			WebApi->UpdatePendingFileList(Options);
			TestTrue(TEXT("Expected updatePendingFileList call"),
				WebApi->TestExpectAsyncFunctionCall(
					*this, nullptr, TEXT("updatePendingFileList"), *Options.ToJson(false)).IsValid());
		});
	});

	Describe("RegisterOnPendingFileDecision", [this]
	{
		It("registers callback without invoking", [this]
		{
			FFakeWebApiContainer WebApi;
			int CallCount = 0;
			auto Future =
				WebApi->RegisterOnPendingFileDecision(
					[&CallCount](FOnPendingFileDecisionOptions Options)
					{
						CallCount += 1;
					});
			const FString& HandlerId = WebApi->RegisteredCallbackHandlerIds.Last();
			WebApi->TestExpectAsyncFunctionCallAndComplete(
				*this,
				nullptr,
				TEXT("registerOnPendingFileDecision"),
				*FWebApiAccessor::FormatFunctionCallback(*WebApi, *HandlerId),
				TEXT(""),
				false);
			TestEqual(TEXT("RegisterOnPendingFileDecision_NoCalls"), CallCount, 0);
		});

		It("invokes callback with accepted decision", [this]
		{
			FFakeWebApiContainer WebApi;
			FString ActualConversationId;
			bool bActualAccepted = false;
			int CallCount = 0;
			auto Result = WebApi->RegisterOnPendingFileDecision(
				[&ActualConversationId, &bActualAccepted, &CallCount](FOnPendingFileDecisionOptions Options)
				{
					ActualConversationId = Options.ConversationId.Id;
					bActualAccepted = Options.bAccepted;
					CallCount += 1;
				});
			const FString& HandlerId = WebApi->RegisteredCallbackHandlerIds.Last();
			if (!WebApi->TestExpectAsyncFunctionCallAndComplete(
				*this,
				nullptr,
				TEXT("registerOnPendingFileDecision"),
				*FWebApiAccessor::FormatFunctionCallback(*WebApi, *HandlerId),
				TEXT(""),
				false))
			{
				return;
			}

			FString ExpectedConversationId = TEXT("conv-123");
			WebApi->InvokeCallback(
				HandlerId,
				TEXT(R"js({ "conversationId": { "id": "conv-123", "type": "ConversationId" }, "accepted": true })js"),
				false);
			TestEqual(
				TEXT("RegisterOnPendingFileDecision_ConversationId"),
				ActualConversationId,
				ExpectedConversationId);
			TestTrue(TEXT("RegisterOnPendingFileDecision_Accepted"), bActualAccepted);
			TestEqual(TEXT("RegisterOnPendingFileDecision_CallCount"), CallCount, 1);
		});

		It("invokes callback with rejected decision", [this]
		{
			FFakeWebApiContainer WebApi;
			bool bActualAccepted = true;
			auto Result = WebApi->RegisterOnPendingFileDecision(
				[&bActualAccepted](FOnPendingFileDecisionOptions Options)
				{
					bActualAccepted = Options.bAccepted;
				});
			const FString& HandlerId = WebApi->RegisteredCallbackHandlerIds.Last();
			if (!WebApi->TestExpectAsyncFunctionCallAndComplete(
				*this,
				nullptr,
				TEXT("registerOnPendingFileDecision"),
				*FWebApiAccessor::FormatFunctionCallback(*WebApi, *HandlerId),
				TEXT(""),
				false))
			{
				return;
			}

			WebApi->InvokeCallback(
				HandlerId,
				TEXT(R"js({ "conversationId": { "id": "conv-1", "type": "ConversationId" }, "accepted": false })js"),
				false);
			TestFalse(TEXT("RegisterOnPendingFileDecision_Rejected"), bActualAccepted);
		});

		It("unregisters via UnregisterOnPendingFileDecision", [this]
		{
			FFakeWebApiContainer WebApi;

			// Register a callback.
			int CallCount = 0;
			auto Result = WebApi->RegisterOnPendingFileDecision(
				[&CallCount](FOnPendingFileDecisionOptions Options)
				{
					CallCount += 1;
				});
			const FString& HandlerId = WebApi->RegisteredCallbackHandlerIds.Last();
			if (!WebApi->TestExpectAsyncFunctionCallAndComplete(
				*this,
				nullptr,
				TEXT("registerOnPendingFileDecision"),
				*FWebApiAccessor::FormatFunctionCallback(*WebApi, *HandlerId),
				TEXT(""),
				false))
			{
				return;
			}

			auto& Registration = Result.Get().GetValue();
			auto& CallbackHandle = Registration.Get<1>();

			// Invoke callback once.
			WebApi->InvokeCallback(
				HandlerId,
				TEXT(R"js({ "conversationId": { "id": "conv-1", "type": "ConversationId" }, "accepted": true })js"),
				false);

			// Unregister the callback.
			WebApi->UnregisterOnPendingFileDecision(Registration);
			if (!WebApi->TestExpectAsyncFunctionCall(
				*this, nullptr, TEXT("unregisterOnPendingFileDecision"),
				*FString::Printf(TEXT("%s"), *CallbackHandle)).IsValid())
			{
				return;
			}

			// Try to invoke callback again.
			WebApi->InvokeCallback(
				HandlerId,
				TEXT(R"js({ "conversationId": { "id": "conv-2", "type": "ConversationId" }, "accepted": false })js"),
				false);

			// Ensure callback was only called the first time.
			TestEqual(TEXT("UnregisterOnPendingFileDecision_CallCount"), CallCount, 1);
		});
	});
}


IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantWebApiTestAddMessageToConversation,
	"AI.Assistant.WebApi.AddMessageToConversation",
	AIAssistantTest::Flags);

bool FAIAssistantWebApiTestAddMessageToConversation::RunTest(const FString& UnusedParameters)
{
	FFakeWebApiContainer WebApi;
	FAddMessageToConversationOptions Options;
	Options.ConversationId.Emplace().Id = TEXT("convo");
	Options.Message.MessageRole = EMessageRole::User;
	FMessageContent& MessageContent = Options.Message.MessageContent.AddDefaulted_GetRef();
	MessageContent.ContentType = EMessageContentType::Text;
	MessageContent.Content.Emplace<FTextMessageContent>();
	MessageContent.Content.Get<FTextMessageContent>().Text = TEXT("Hello");
	WebApi->AddMessageToConversation(Options);
	return WebApi->TestExpectAsyncFunctionCall(
		*this, nullptr, TEXT("addMessageToConversation"), *Options.ToJson(false)).IsValid();
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantWebApiTestAddAgentEnvironment,
	"AI.Assistant.WebApi.AddAgentEnvironment",
	AIAssistantTest::Flags);

bool FAIAssistantWebApiTestAddAgentEnvironment::RunTest(const FString& UnusedParameters)
{
	FFakeWebApiContainer WebApi;
	FAgentEnvironment AgentEnvironment;
	AgentEnvironment.Descriptor.EnvironmentName = TEXT("UE");
	AgentEnvironment.Descriptor.EnvironmentVersion = TEXT("5.7.0");
	auto Result = WebApi->AddAgentEnvironment(AgentEnvironment);

	FAgentEnvironmentHandle AgentEnvironmentHandle;
	AgentEnvironmentHandle.Id.Id = TEXT("fakeId");
	AgentEnvironmentHandle.Hash.Hash = TEXT("fakeHash");

	return WebApi->TestExpectAsyncFunctionCallAndComplete(
		*this, nullptr, TEXT("addAgentEnvironment"), *AgentEnvironment.ToJson(false),
		Result, *AgentEnvironmentHandle.ToJson(false), false);
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantWebApiTestAddAgentEnvironmentFailed,
	"AI.Assistant.WebApi.AddAgentEnvironmentFailed",
	AIAssistantTest::Flags);

bool FAIAssistantWebApiTestAddAgentEnvironmentFailed::RunTest(const FString& UnusedParameters)
{
	FFakeWebApiContainer WebApi;
	FAgentEnvironment AgentEnvironment;
	auto Result = WebApi->AddAgentEnvironment(AgentEnvironment);

	return WebApi->TestExpectAsyncFunctionCallAndComplete(
		*this, nullptr, TEXT("addAgentEnvironment"), *AgentEnvironment.ToJson(false),
		Result, TEXT("failed"), true);
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantWebApiTestSetAgentEnvironment,
	"AI.Assistant.WebApi.SetAgentEnvironment",
	AIAssistantTest::Flags);

bool FAIAssistantWebApiTestSetAgentEnvironment::RunTest(const FString& UnusedParameters)
{
	FFakeWebApiContainer WebApi;
	FAgentEnvironmentId Id;
	Id.Id = TEXT("fakeId");
	WebApi->SetAgentEnvironment(Id);
	return WebApi->TestExpectAsyncFunctionCall(
		*this, nullptr, TEXT("setAgentEnvironment"), *Id.ToJson(false)).IsValid();
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantWebApiTestUpdateGlobalLocale,
	"AI.Assistant.WebApi.UpdateGlobalLocale",
	AIAssistantTest::Flags);

bool FAIAssistantWebApiTestUpdateGlobalLocale::RunTest(const FString& UnusedParameters)
{
	FFakeWebApiContainer WebApi;
	FString LocaleFromSetting = TEXT("fr");
	WebApi->UpdateGlobalLocale(LocaleFromSetting);
	return WebApi->TestExpectAsyncFunctionCall(
		*this, nullptr, TEXT("updateGlobalLocale"), *FString::Printf(TEXT("\"%s\""),
		*LocaleFromSetting)).IsValid();
}


IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantWebApiTestRegisterOnConversationUpdate,
	"AI.Assistant.WebApi.RegisterOnConversationUpdate",
	AIAssistantTest::Flags);

bool FAIAssistantWebApiTestRegisterOnConversationUpdate::RunTest(const FString& UnusedParameters)
{
	FFakeWebApiContainer WebApi;
	int CallCount = 0;
	auto Future =
		WebApi->RegisterOnConversationUpdate([&CallCount](FConversationUpdateEvent Event)
		{
			CallCount += 1;
			return UAIAssistantWebJavaScriptResultDelegate::EHandlerStatus::Continue;
		});
	const FString& HandlerId = WebApi->RegisteredCallbackHandlerIds.Last();
	return
		WebApi->TestExpectAsyncFunctionCallAndComplete(
			*this,
			nullptr,
			TEXT("registerOnConversationUpdate"),
			*FWebApiAccessor::FormatFunctionCallback(*WebApi, *HandlerId),
			TEXT(""),
			false)
		&& TestEqual(TEXT("RegisterOnConversationUpdate_NoCalls"), CallCount, 0);
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantWebApiTestRegisterOnConversationUpdateAndInvoke,
	"AI.Assistant.WebApi.RegisterOnConversationUpdateAndInvoke",
	AIAssistantTest::Flags);

bool FAIAssistantWebApiTestRegisterOnConversationUpdateAndInvoke::RunTest(
	const FString& UnusedParameters)
{
	FFakeWebApiContainer WebApi;
	FString ActualConversationId;
	int CallCount = 0;
	auto Result = WebApi->RegisterOnConversationUpdate(
		[&ActualConversationId, &CallCount](FConversationUpdateEvent Event)
		{
			ActualConversationId = Event.ConversationId.Id;
			CallCount += 1;
			return UAIAssistantWebJavaScriptResultDelegate::EHandlerStatus::Continue;
		});
	const FString& HandlerId = WebApi->RegisteredCallbackHandlerIds.Last();
	if (!WebApi->TestExpectAsyncFunctionCallAndComplete(
		*this,
		nullptr,
		TEXT("registerOnConversationUpdate"),
		*FWebApiAccessor::FormatFunctionCallback(*WebApi, *HandlerId),
		TEXT(""),
		false))
	{
		return false;
	}

	auto& Registration = Result.Get().GetValue();
	auto& CallbackHandle = Registration.Get<1>();

	FString ExpectedConversationId = TEXT("foobar");
	WebApi->InvokeCallback(
		HandlerId,
		*MakeConversationUpdateEventJson(*ExpectedConversationId),
		false);
	return
		TestEqual(
			TEXT("RegisterOnConversationUpdate_ConversationId"),
			ActualConversationId,
			ExpectedConversationId)
		&& TestEqual(TEXT("RegisterOnConversationUpdate_CallCount"), CallCount, 1);
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantWebApiTestRegisterOnConversationUpdateAndInvokeTwice,
	"AI.Assistant.WebApi.RegisterOnConversationUpdateAndInvokeTwice",
	AIAssistantTest::Flags);

bool FAIAssistantWebApiTestRegisterOnConversationUpdateAndInvokeTwice::RunTest(
	const FString& UnusedParameters)
{
	FFakeWebApiContainer WebApi;
	TArray<FString> ActualConversationIds;
	int CallCount = 0;
	auto Result = WebApi->RegisterOnConversationUpdate(
		[&ActualConversationIds, &CallCount](FConversationUpdateEvent Event)
		{
			ActualConversationIds.Push(Event.ConversationId.Id);
			CallCount += 1;
			return UAIAssistantWebJavaScriptResultDelegate::EHandlerStatus::Continue;
		});

	const FString& HandlerId = WebApi->RegisteredCallbackHandlerIds.Last();

	if (!WebApi->TestExpectAsyncFunctionCallAndComplete(
		*this,
		nullptr,
		TEXT("registerOnConversationUpdate"),
		*FWebApiAccessor::FormatFunctionCallback(*WebApi, *HandlerId),
		TEXT(""),
		false))
	{
		return false;
	}

	auto& Registration = Result.Get().GetValue();
	auto& CallbackHandle = Registration.Get<1>();

	TArray<FString> ExpectedConversationIds = { TEXT("foo"), TEXT("bar") };
	for (const auto& ConversationId : ExpectedConversationIds)
	{
		WebApi->InvokeCallback(
			HandlerId,
			*MakeConversationUpdateEventJson(*ConversationId),
			false);
	}
	return
		TestEqual(
			TEXT("RegisterOnConversationUpdateTwice_ConversationIds"),
			ActualConversationIds,
			ExpectedConversationIds)
		&& TestEqual(TEXT("RegisterOnConversationUpdateTwice_CallCount"), CallCount, 2);
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantWebApiTestUnegisterOnConversationUpdateViaReturnValue,
	"AI.Assistant.WebApi.UnregisterOnConversationUpdateViaReturnValue",
	AIAssistantTest::Flags);


bool FAIAssistantWebApiTestUnegisterOnConversationUpdateViaReturnValue::RunTest(
	const FString& UnusedParameters)
{
	FFakeWebApiContainer WebApi;
	TArray<FString> ActualConversationIds;
	int CallCount = 0;
	auto Result = WebApi->RegisterOnConversationUpdate(
		[&ActualConversationIds, &CallCount](FConversationUpdateEvent Event)
		{
			ActualConversationIds.Push(Event.ConversationId.Id);
			CallCount += 1;
			return UAIAssistantWebJavaScriptResultDelegate::EHandlerStatus::Unregister;
		});
	const FString& HandlerId = WebApi->RegisteredCallbackHandlerIds.Last();
	if (!WebApi->TestExpectAsyncFunctionCallAndComplete(
		*this,
		nullptr,
		TEXT("registerOnConversationUpdate"),
		*FWebApiAccessor::FormatFunctionCallback(*WebApi, *HandlerId),
		TEXT(""),
		false))
	{
		return false;
	}

	auto& Registration = Result.Get().GetValue();
	auto& CallbackHandle = Registration.Get<1>();

	TArray<FString> ExpectedConversationIds = { TEXT("foo") };
	for (const auto& ConversationId : ExpectedConversationIds)
	{
		WebApi->InvokeCallback(
			*HandlerId,
			*MakeConversationUpdateEventJson(*ConversationId),
			false);
	}
	return
		TestEqual(
			TEXT("RegisterOnConversationUpdateTwice_ConversationIds"),
			ActualConversationIds,
			ExpectedConversationIds)
		&& TestEqual(TEXT("RegisterOnConversationUpdateTwice_CallCount"), CallCount, 1);
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantWebApiTestUnregisterOnConversationUpdate,
	"AI.Assistant.WebApi.UnregisterOnConversationUpdate",
	AIAssistantTest::Flags);
bool FAIAssistantWebApiTestUnregisterOnConversationUpdate::RunTest(const FString& UnusedParameters)
{
	FFakeWebApiContainer WebApi;

	// Register a callback.
	TArray<FString> ActualConversationIds;
	int CallCount = 0;
	auto Result = WebApi->RegisterOnConversationUpdate(
		[&ActualConversationIds, &CallCount](FConversationUpdateEvent Event)
		{
			ActualConversationIds.Push(Event.ConversationId.Id);
			CallCount += 1;
			return UAIAssistantWebJavaScriptResultDelegate::EHandlerStatus::Continue;
		});
	const FString& HandlerId = WebApi->RegisteredCallbackHandlerIds.Last();
	if (!WebApi->TestExpectAsyncFunctionCallAndComplete(
		*this,
		nullptr,
		TEXT("registerOnConversationUpdate"),
		*FWebApiAccessor::FormatFunctionCallback(*WebApi, *HandlerId),
		TEXT(""),
		false))
	{
		return false;
	}

	auto& Registration = Result.Get().GetValue();
	auto& CallbackHandle = Registration.Get<1>();

	// Invoke callback once.
	WebApi->InvokeCallback(
		HandlerId,
		*MakeConversationUpdateEventJson(TEXT("foo")),
		false);

	// Unregister the callback.
	WebApi->UnregisterOnConversationUpdate(Registration);
	if (!WebApi->TestExpectAsyncFunctionCall(
		*this, nullptr, TEXT("unregisterOnConversationUpdate"),
		*FString::Printf(TEXT("%s"), *CallbackHandle)).IsValid())
	{
		return false;
	}

	// Try to invoke callback again.
	WebApi->InvokeCallback(
		HandlerId,
		*MakeConversationUpdateEventJson(TEXT("bar")),
		false);

	// Ensure callback was only called the first time.
	TArray<FString> ExpectedConversationIds = { TEXT("foo") };
	return
		TestEqual(
			TEXT("UnregisterOnConversationUpdate_IdsMatch"),
			ActualConversationIds,
			ExpectedConversationIds)
		&& TestEqual(TEXT("UnregisterOnConversationUpdate_CallCount"), CallCount, 1);
}

#endif  // WITH_DEV_AUTOMATION_TESTS
