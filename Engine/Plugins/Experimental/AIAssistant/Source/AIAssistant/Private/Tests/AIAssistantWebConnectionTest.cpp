// Copyright Epic Games, Inc. All Rights Reserved.
	

#include "AIAssistantTestFlags.h"

#include "AIAssistantWebConnectionWidget.h"


#if WITH_DEV_AUTOMATION_TESTS


//
// FAIAssistantFakeWebConnectionWidgetTester
//
// Contains a SAIAssistantWebConnectionWidget.
// Also contains tracking of event calls made to provided lambdas, and other helpers for testing.
//


class FAIAssistantFakeWebConnectionWidgetTester
{
public:

	
	enum class EFakeWebPageLoadState : uint8
	{
		LoadPending = 0,
		LoadError,
		LoadComplete
	};


	/**
	 * @param InAutomationTestBase The test we're using this in. 
	 * @param InFakeWebPageLoadState The simulated web page load state to assume during this test.
	 */
	FAIAssistantFakeWebConnectionWidgetTester(FAutomationTestBase& InAutomationTestBase, const EFakeWebPageLoadState InFakeWebPageLoadState) :
		AutomationTestBase(InAutomationTestBase),
		FakeWebPageLoadState(InFakeWebPageLoadState)
	{
		SAssignNew(AIAssistantWebConnectionWidget, SAIAssistantWebConnectionWidget)
			.bUseReconnectingTicker(false) // ..we can't use the ticker during these tests
			.OnRequestConnectionState([this]() -> SAIAssistantWebConnectionWidget::EConnectionState
				{
					// This lambda simulates how SAIAssistantWebBrowser would react to being told to reconnect.
					// We have a fake state that simulates whether connection is possible.
			
					if (FakeWebPageLoadState == EFakeWebPageLoadState::LoadError)
					{
						return SAIAssistantWebConnectionWidget::EConnectionState::Disconnected;
					}
					else if (FakeWebPageLoadState == EFakeWebPageLoadState::LoadComplete)
					{
						return SAIAssistantWebConnectionWidget::EConnectionState::Connected;
					}
					else
					{
						return SAIAssistantWebConnectionWidget::EConnectionState::Reconnecting;
					}
				})
			.OnConnected_Lambda([this]() -> void
				{
					++ConnectedEventCount;
				})
			.OnDisconnected_Lambda([this]() -> void
				{
					++DisconnectedEventCount;
				})
			.OnReconnect_Lambda([this]() -> void
				{
					++ReconnectEventCount;
				});
	}


	/**
	 * @return Internal SAIAssistantWebConnectionWidget used for testing.  
	 */
	TSharedPtr<SAIAssistantWebConnectionWidget> GetAIAssistantWebConnectionWidget() const
	{
		return AIAssistantWebConnectionWidget;
	}


	/**
	 * @return Returns current testing step counter, used to control different phases to a test.
	 */
	unsigned int GetTestStep() const
	{
		return TestStep;
	}

	/**
	 * Updates reconnecting state (since the SAIAssistantWebConnectionWidget is not using a ticker), and advances the testing step counter.
	 */
	void UpdateReconnectingAndAdvanceTestStep()
	{
		AIAssistantWebConnectionWidget->UpdateReconnecting();
		
		TestStep++;	
	}


	/**
	 * Checks expected number of event (lambda call) counts against the number of actual event (lambda call) counts we received during testing.
	 * @param ExpectedConnectedEventCount Number of expected connected events.
	 * @param ExpectedDisconnectedEventCount Number of expected disconnected events.
	 * @param ExpectedReconnectEventCount  Number of expected reconnect events.
	 * @return Whether all counts.
	 */
	bool TestConnectionEventCounts(
		const unsigned int ExpectedConnectedEventCount,
		const unsigned int ExpectedDisconnectedEventCount,
		const unsigned int ExpectedReconnectEventCount) const
	{
		auto GetTestWhat = [&](const FString& ValueName) -> FString
			{
				return FString::Printf(TEXT("Test_%d_%s"), TestStep, *ValueName);		
			};

		bool bOk = true;
		bOk &= AutomationTestBase.TestEqual(GetTestWhat("ConnectedEventCount"), ConnectedEventCount, ExpectedConnectedEventCount);
		bOk &= AutomationTestBase.TestEqual(GetTestWhat("DisconnectedEventCount"), DisconnectedEventCount, ExpectedDisconnectedEventCount);
		bOk &= AutomationTestBase.TestEqual(GetTestWhat("ReconnectEventCount"), ReconnectEventCount, ExpectedReconnectEventCount);
	
		return bOk;
	}

	
private:

	
	FAutomationTestBase& AutomationTestBase;
	TSharedPtr<SAIAssistantWebConnectionWidget> AIAssistantWebConnectionWidget;
	EFakeWebPageLoadState FakeWebPageLoadState = EFakeWebPageLoadState::LoadPending;
	unsigned int TestStep = 0;
	unsigned int DisconnectedEventCount = 0, ConnectedEventCount = 0, ReconnectEventCount = 0;
};


//
// AIAssistantWebConnectionTestDisconnectTwice
//


IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	AIAssistantWebConnectionTestDisconnectTwice,
	"AI.Assistant.WebConnection.DisconnectTwice",
	AIAssistantTest::Flags);

bool AIAssistantWebConnectionTestDisconnectTwice::RunTest(const FString& Parameters)
{
	// Disconnect (twice). (Load state doesn't matter.)

	FAIAssistantFakeWebConnectionWidgetTester Tester(*this, 
		FAIAssistantFakeWebConnectionWidgetTester::EFakeWebPageLoadState::LoadComplete/*but doesn't matter here*/);
	while (true)
	{
		if (Tester.GetTestStep() == 0)
		{
			(void) Tester.TestConnectionEventCounts(0, 0, 0);

			Tester.GetAIAssistantWebConnectionWidget()->Disconnect();
			Tester.UpdateReconnectingAndAdvanceTestStep();
		}
		else if (Tester.GetTestStep() == 1)
		{
			(void) Tester.TestConnectionEventCounts(0, 1, 0);

			Tester.GetAIAssistantWebConnectionWidget()->Disconnect();
			Tester.UpdateReconnectingAndAdvanceTestStep();
		}
		else if (Tester.GetTestStep() == 2)
		{
			(void) Tester.TestConnectionEventCounts(0, 1, 0);

			Tester.UpdateReconnectingAndAdvanceTestStep();
		}
		else
		{
			break;
		}
	}

	
	return true;
}


//
// AIAssistantWebConnectionTestReconnectTwiceWhenLoadComplete
//


IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	AIAssistantWebConnectionTestReconnectTwiceWhenLoadComplete,
	"AI.Assistant.WebConnection.ReconnectTwiceWhenLoadComplete",
	AIAssistantTest::Flags);

bool AIAssistantWebConnectionTestReconnectTwiceWhenLoadComplete::RunTest(const FString& Parameters)
{
	// Reconnect (twice) in 'load complete' state.

	FAIAssistantFakeWebConnectionWidgetTester Tester(*this,
		FAIAssistantFakeWebConnectionWidgetTester::EFakeWebPageLoadState::LoadComplete);
	while (true)
	{
		if (Tester.GetTestStep() == 0)
		{
			(void) Tester.TestConnectionEventCounts(0, 0, 0);
		
			Tester.GetAIAssistantWebConnectionWidget()->Disconnect();
			Tester.UpdateReconnectingAndAdvanceTestStep();
		}
		else if (Tester.GetTestStep() == 1)
		{
			(void) Tester.TestConnectionEventCounts(0, 1, 0);
		
			Tester.GetAIAssistantWebConnectionWidget()->StartReconnecting();
			Tester.UpdateReconnectingAndAdvanceTestStep();
		}
		else if (Tester.GetTestStep() == 2)
		{
			(void) Tester.TestConnectionEventCounts(1, 1, 1);
		
			Tester.GetAIAssistantWebConnectionWidget()->StartReconnecting();
			Tester.UpdateReconnectingAndAdvanceTestStep();
		}
		else if (Tester.GetTestStep() == 3)
		{
			(void) Tester.TestConnectionEventCounts(1, 1, 1);
			
			Tester.UpdateReconnectingAndAdvanceTestStep();
		}
		else
		{
			break;
		}
	}


	return true;
};


//
// AIAssistantWebConnectionTestReconnectTwiceWhenLoadError
//


IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	AIAssistantWebConnectionTestReconnectTwiceWhenLoadError,
	"AI.Assistant.WebConnection.ReconnectTwiceWhenLoadError",
	AIAssistantTest::Flags);

bool AIAssistantWebConnectionTestReconnectTwiceWhenLoadError::RunTest(const FString& Parameters)
{
	// Reconnect (twice) in 'load error' state.

	FAIAssistantFakeWebConnectionWidgetTester Tester(*this,
		FAIAssistantFakeWebConnectionWidgetTester::EFakeWebPageLoadState::LoadError);
	while (true)
	{
		if (Tester.GetTestStep() == 0)
		{
			(void) Tester.TestConnectionEventCounts(0, 0, 0);
		
			Tester.GetAIAssistantWebConnectionWidget()->Disconnect();
			Tester.UpdateReconnectingAndAdvanceTestStep();
		}
		else if (Tester.GetTestStep() == 1)
		{
			(void) Tester.TestConnectionEventCounts(0, 1, 0);
		
			Tester.GetAIAssistantWebConnectionWidget()->StartReconnecting();
			Tester.UpdateReconnectingAndAdvanceTestStep();
		}
		else if (Tester.GetTestStep() == 2)
		{
			(void) Tester.TestConnectionEventCounts(0, 2, 1);
		
			Tester.GetAIAssistantWebConnectionWidget()->StartReconnecting();
			Tester.UpdateReconnectingAndAdvanceTestStep();
		}
		else if (Tester.GetTestStep() == 3)
		{
			(void) Tester.TestConnectionEventCounts(0, 3, 2);
			
			Tester.UpdateReconnectingAndAdvanceTestStep();
		}
		else
		{
			break;
		}
	}

	
	return true;
}


//
// AIAssistantWebConnectionTestReconnectTwiceWhenLoadPending
//


IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	AIAssistantWebConnectionTestReconnectTwiceWhenLoadPending,
	"AI.Assistant.WebConnection.ReconnectTwiceWhenLoadPending",
	AIAssistantTest::Flags);

bool AIAssistantWebConnectionTestReconnectTwiceWhenLoadPending::RunTest(const FString& Parameters)
{
	// Reconnect (twice) in 'load pending' state.

	FAIAssistantFakeWebConnectionWidgetTester Tester(*this,
		FAIAssistantFakeWebConnectionWidgetTester::EFakeWebPageLoadState::LoadPending);
	while (true)
	{
		if (Tester.GetTestStep() == 0)
		{
			(void) Tester.TestConnectionEventCounts(0, 0, 0);
		
			Tester.GetAIAssistantWebConnectionWidget()->Disconnect();
			Tester.UpdateReconnectingAndAdvanceTestStep();
		}
		else if (Tester.GetTestStep() == 1)
		{
			(void) Tester.TestConnectionEventCounts(0, 1, 0);
		
			Tester.GetAIAssistantWebConnectionWidget()->StartReconnecting();
			Tester.UpdateReconnectingAndAdvanceTestStep();
		}
		else if (Tester.GetTestStep() == 2)
		{
			(void) Tester.TestConnectionEventCounts(0, 1, 1);
		
			Tester.GetAIAssistantWebConnectionWidget()->StartReconnecting();
			Tester.UpdateReconnectingAndAdvanceTestStep();
		}
		else if (Tester.GetTestStep() == 3)
		{
			(void) Tester.TestConnectionEventCounts(0, 1, 1);
		
			Tester.UpdateReconnectingAndAdvanceTestStep();
		}
		else
		{
			break;
		}
	}

	
	return true;
}


#endif
