// Copyright Epic Games, Inc. All Rights Reserved.

#include "Containers/UnrealString.h"
#include "Misc/AutomationTest.h"
#include "UObject/Object.h"
#include "UObject/StrongObjectPtr.h"

#include "AIAssistantTestFlags.h"
#include "AIAssistantTestObject.h"
#include "AIAssistantFakeWebJavaScriptDelegateBinder.h"
#include "AIAssistantWebJavaScriptDelegateBinder.h"

#if WITH_DEV_AUTOMATION_TESTS

using namespace UE::AIAssistant;

// Create a UObject, the type doesn't matter for these tests.
static TStrongObjectPtr<UAIAssistantTestObject> CreateUObject()
{
	return TStrongObjectPtr<UAIAssistantTestObject>(NewObject<UAIAssistantTestObject>());
}

const FString ObjectName(TEXT("ObjectToBind"));

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantWebJavaScriptDelegateBinderTestBindUnbind,
	"AI.Assistant.WebJavaScriptDelegateBinder.BindUnbind",
	AIAssistantTest::Flags);

bool FAIAssistantWebJavaScriptDelegateBinderTestBindUnbind::RunTest(
	const FString& UnusedParameters)
{
	auto Object = CreateUObject();
	FFakeWebJavaScriptDelegateBinder Binder;
	Binder.BindUObject(ObjectName, Object.Get());
	(void)TestEqual(TEXT("NumberOfBoundObjects"), 1, Binder.BoundObjects.Num());
	Binder.UnbindUObject(ObjectName, Object.Get());
	(void)TestEqual(TEXT("NumberOfBoundObjects"), 0, Binder.BoundObjects.Num());
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantScopedWebJavaScriptDelegateBinderTestBindUnbind,
	"AI.Assistant.ScopedWebJavaScriptDelegateBinder.BindUnbind",
	AIAssistantTest::Flags);

bool FAIAssistantScopedWebJavaScriptDelegateBinderTestBindUnbind::RunTest(
	const FString& UnusedParameters)
{
	auto Object = CreateUObject();
	FFakeWebJavaScriptDelegateBinder Binder;
	{
		FScopedWebJavaScriptDelegateBinder ScopedBinder(Binder, ObjectName, Object.Get());
		(void)TestEqual(TEXT("NumberOfBoundObjects"), 1, Binder.BoundObjects.Num());
	}
	(void)TestEqual(TEXT("NumberOfBoundObjects"), 0, Binder.BoundObjects.Num());
	return true;
}

#endif  // WITH_DEV_AUTOMATION_TESTS