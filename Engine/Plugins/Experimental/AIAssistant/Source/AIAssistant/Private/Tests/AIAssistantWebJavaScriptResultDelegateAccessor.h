// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Templates/SharedPointer.h"

#include "AIAssistantWebJavaScriptResultDelegate.h"

namespace UE::AIAssistant
{
	class FWebJavaScriptResultDelegateAccessor
	{
	public:
		// Call Delegate.HandleResult().
		static void CallHandleResult(
			UAIAssistantWebJavaScriptResultDelegate& Delegate,
			const FString& HandlerId, const FString& ResultJson, bool bResultJsonIsError)
		{
			Delegate.HandleResult(HandlerId, ResultJson, bResultJsonIsError);
		}

		static const TMap<
			FString, TSharedPtr<UAIAssistantWebJavaScriptResultDelegate::FResultHandler>>&
			GetResultHandlersById(UAIAssistantWebJavaScriptResultDelegate& Delegate)
		{
			return Delegate.ResultHandlersById;
		}
	};
}