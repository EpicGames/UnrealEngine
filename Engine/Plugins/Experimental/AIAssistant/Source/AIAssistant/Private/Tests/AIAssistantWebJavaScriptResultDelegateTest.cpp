// Copyright Epic Games, Inc. All Rights Reserved.

#include "Async/Future.h"
#include "Containers/StaticArray.h"
#include "Containers/UnrealString.h"
#include "Delegates/DelegateCombinations.h"
#include "Misc/AutomationTest.h"
#include "Templates/Tuple.h"
#include "UObject/Object.h"
#include "UObject/StrongObjectPtr.h"

#include "AIAssistantTestFlags.h"
#include "AIAssistantFakeWebJavaScriptDelegateBinder.h"
#include "AIAssistantWebJavaScriptResultDelegate.h"
#include "AIAssistantWebJavaScriptResultDelegateAccessor.h"

#if WITH_DEV_AUTOMATION_TESTS

using namespace UE::AIAssistant;

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantWebJavaScriptResultDelegateTestGetName,
	"AI.Assistant.WebJavaScriptResultDelegate.GetName",
	AIAssistantTest::Flags);

bool FAIAssistantWebJavaScriptResultDelegateTestGetName::RunTest(
	const FString& UnusedParameters)
{
	TStrongObjectPtr<UAIAssistantWebJavaScriptResultDelegate> ResultDelegate(
		NewObject<UAIAssistantWebJavaScriptResultDelegate>());
	FString Name = ResultDelegate->GetName();
	(void)TestEqual(
		TEXT("NameStartsWithBaseName"),
		Name.Left(UAIAssistantWebJavaScriptResultDelegate::BaseName.Len()),
		UAIAssistantWebJavaScriptResultDelegate::BaseName);
	(void)TestNotEqual(
		TEXT("NameHasUniqueTail"),
		Name.Right(
			Name.Len() -
			Name.Left(UAIAssistantWebJavaScriptResultDelegate::BaseName.Len()).Len()),
		TEXT(""));

	const FString& AnotherName = ResultDelegate->GetName();
	(void)TestEqual(TEXT("NameDoesNotChange"), Name, AnotherName);
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantWebJavaScriptResultDelegateTestBindUnbind,
	"AI.Assistant.WebJavaScriptResultDelegate.BindUnbind",
	AIAssistantTest::Flags);

bool FAIAssistantWebJavaScriptResultDelegateTestBindUnbind::RunTest(
	const FString& UnusedParameters)
{
	FFakeWebJavaScriptDelegateBinder Binder;
	TStrongObjectPtr<UAIAssistantWebJavaScriptResultDelegate> ResultDelegate(
		NewObject<UAIAssistantWebJavaScriptResultDelegate>());
	ResultDelegate->Bind(Binder);
	(void)TestEqual(TEXT("NumberOfBoundObjects"), 1, Binder.BoundObjects.Num());
	ResultDelegate->Unbind();
	(void)TestEqual(TEXT("NumberOfBoundObjects"), 0, Binder.BoundObjects.Num());
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantWebJavaScriptResultDelegateTestRegisterResultHandler,
	"AI.Assistant.WebJavaScriptResultDelegate.RegisterResultHandler",
	AIAssistantTest::Flags);

bool FAIAssistantWebJavaScriptResultDelegateTestRegisterResultHandler::RunTest(
	const FString& UnusedParameters)
{
	TStrongObjectPtr<UAIAssistantWebJavaScriptResultDelegate> ResultDelegate(
		NewObject<UAIAssistantWebJavaScriptResultDelegate>());

	FString HandlerId = ResultDelegate->RegisterResultHandler(
		[](UAIAssistantWebJavaScriptResultDelegate::FResultHandlerContext&&) ->
			UAIAssistantWebJavaScriptResultDelegate::EHandlerStatus
		{
			return UAIAssistantWebJavaScriptResultDelegate::EHandlerStatus::Unregister;
		});
	(void)TestNotEqual(TEXT("HandlerId"), TEXT(""), HandlerId);
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantWebJavaScriptResultDelegateTestCallResultHandler,
	"AI.Assistant.WebJavaScriptResultDelegate.CallResultHandler",
	AIAssistantTest::Flags);

bool FAIAssistantWebJavaScriptResultDelegateTestCallResultHandler::RunTest(
	const FString& UnusedParameters)
{
	TStrongObjectPtr<UAIAssistantWebJavaScriptResultDelegate> ResultDelegate(
		NewObject<UAIAssistantWebJavaScriptResultDelegate>());

	for (bool bExpectedJsonResultIsError : TStaticArray<bool, 2>{ true, false })
	{
		const FString ExpectedJsonResult = TEXT("{'answer': 42}");
		bool bCalledHandler = false;
		FString HandlerId = ResultDelegate->RegisterResultHandler(
			[this, &ExpectedJsonResult, bExpectedJsonResultIsError, &bCalledHandler](
				UAIAssistantWebJavaScriptResultDelegate::FResultHandlerContext&&
				ResultHandlerContext) -> UAIAssistantWebJavaScriptResultDelegate::EHandlerStatus
			{
				(void)TestEqual(
					TEXT("JsonResult"), ExpectedJsonResult,
					ResultHandlerContext.Json);
				(void)TestEqual(
					TEXT("bJsonResultIsError"), bExpectedJsonResultIsError,
					ResultHandlerContext.bJsonIsError);
				bCalledHandler = true;
				return UAIAssistantWebJavaScriptResultDelegate::EHandlerStatus::Unregister;
			});

		FWebJavaScriptResultDelegateAccessor::CallHandleResult(
			*ResultDelegate, HandlerId, ExpectedJsonResult, bExpectedJsonResultIsError);
		(void)TestTrue(TEXT("CalledHandler"), bCalledHandler);
	}
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantWebJavaScriptResultDelegateTestCallResultHandlerOnce,
	"AI.Assistant.WebJavaScriptResultDelegate.CallResultHandlerOnce",
	AIAssistantTest::Flags);

bool FAIAssistantWebJavaScriptResultDelegateTestCallResultHandlerOnce::RunTest(
	const FString& UnusedParameters)
{
	TStrongObjectPtr<UAIAssistantWebJavaScriptResultDelegate> ResultDelegate(
		NewObject<UAIAssistantWebJavaScriptResultDelegate>());

	bool bWasCalled = false;
	FString HandlerId = ResultDelegate->RegisterResultHandler(
		[&bWasCalled](UAIAssistantWebJavaScriptResultDelegate::FResultHandlerContext&&) ->
			UAIAssistantWebJavaScriptResultDelegate::EHandlerStatus
		{
			bWasCalled = true;
			return UAIAssistantWebJavaScriptResultDelegate::EHandlerStatus::Unregister;
		});
	(void)TestEqual(
		TEXT("NumberOfHandlers"), 1,
		FWebJavaScriptResultDelegateAccessor::GetResultHandlersById(
			*ResultDelegate).Num());

	FWebJavaScriptResultDelegateAccessor::CallHandleResult(
		*ResultDelegate, HandlerId, TEXT(""), false);
	(void)TestTrue(TEXT("HandlerCalled"), bWasCalled);
	(void)TestEqual(
		TEXT("NumberOfHandlers"), 0,
		FWebJavaScriptResultDelegateAccessor::GetResultHandlersById(
			*ResultDelegate).Num());
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantWebJavaScriptResultDelegateTestCallResultHandlerMultiple,
	"AI.Assistant.WebJavaScriptResultDelegate.CallResultHandlerMultiple",
	AIAssistantTest::Flags);

bool FAIAssistantWebJavaScriptResultDelegateTestCallResultHandlerMultiple::RunTest(
	const FString& UnusedParameters)
{
	TStrongObjectPtr<UAIAssistantWebJavaScriptResultDelegate> ResultDelegate(
		NewObject<UAIAssistantWebJavaScriptResultDelegate>());

	int NumberOfCalls = 0;
	FString HandlerId = ResultDelegate->RegisterResultHandler(
		[&NumberOfCalls](
			UAIAssistantWebJavaScriptResultDelegate::FResultHandlerContext&&) mutable ->
			UAIAssistantWebJavaScriptResultDelegate::EHandlerStatus
		{
			NumberOfCalls++;
			return UAIAssistantWebJavaScriptResultDelegate::EHandlerStatus::Continue;
		});

	for (int i = 0; i < 2; ++i)
	{
		(void)TestEqual(TEXT("ExpectedNumberOfCallsBefore"), i, NumberOfCalls);
		FWebJavaScriptResultDelegateAccessor::CallHandleResult(
			*ResultDelegate, HandlerId, TEXT(""), false);

		(void)TestEqual(TEXT("ExpectedNumberOfCallsAfter"), i + 1, NumberOfCalls);
	}

	(void)TestEqual(
		TEXT("NumberOfHandlers"), 1,
		FWebJavaScriptResultDelegateAccessor::GetResultHandlersById(
			*ResultDelegate).Num());
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantWebJavaScriptResultDelegateTestFormatJavaScriptHandler,
	"AI.Assistant.WebJavaScriptResultDelegate.FormatJavaScriptHandler",
	AIAssistantTest::Flags);

bool FAIAssistantWebJavaScriptResultDelegateTestFormatJavaScriptHandler::RunTest(
	const FString& UnusedParameters)
{
	TStrongObjectPtr<UAIAssistantWebJavaScriptResultDelegate> ResultDelegate(
		NewObject<UAIAssistantWebJavaScriptResultDelegate>());
	FString HandlerId = TEXT("fake_handler_id");
	(void)TestEqual(
		TEXT("JavaScript"),
		ResultDelegate->FormatJavaScriptHandler(HandlerId),
		FString::Printf(
			TEXT(
				R"js(window.ue.%s.handleresult)js"
				R"js(("fake_handler_id", JSON.stringify({0}), {1});)js"),
			*ResultDelegate->GetName()));
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantWebJavaScriptResultDelegateTestRegisterResultHandlerForFuture,
	"AI.Assistant.WebJavaScriptResultDelegate.RegisterResultHandlerForFuture",
	AIAssistantTest::Flags);

bool FAIAssistantWebJavaScriptResultDelegateTestRegisterResultHandlerForFuture::RunTest(
	const FString& UnusedParameters)
{
	TStrongObjectPtr<UAIAssistantWebJavaScriptResultDelegate> ResultDelegate(
		NewObject<UAIAssistantWebJavaScriptResultDelegate>());
	TPair<FString, TFuture<UAIAssistantWebJavaScriptResultDelegate::FResult>> ResultHandle =
		ResultDelegate->RegisterResultHandlerForFuture();
	(void)TestNotEqual(TEXT("HandlerId"), TEXT(""), ResultHandle.Key);
	(void)TestFalse(TEXT("FutureNotReady"), ResultHandle.Value.IsReady());
	
	const FString FakeResultJson(TEXT("result"));
	const bool bResultJsonIsError = false;
	FWebJavaScriptResultDelegateAccessor::CallHandleResult(
		*ResultDelegate, ResultHandle.Key, FakeResultJson, bResultJsonIsError);

	(void)TestTrue(TEXT("FutureIsReady"), ResultHandle.Value.IsReady());
	auto Result = ResultHandle.Value.Consume();
	(void)TestEqual(TEXT("ResultJson"), FakeResultJson, Result.Json);
	(void)TestEqual(TEXT("ResultJsonIsError"), bResultJsonIsError, Result.bJsonIsError);
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantWebJavaScriptResultDelegateTestRegisterResultHandlerForCallback,
	"AI.Assistant.WebJavaScriptResultDelegate.RegisterResultHandlerForCallback",
	AIAssistantTest::Flags);

bool FAIAssistantWebJavaScriptResultDelegateTestRegisterResultHandlerForCallback::RunTest(
	const FString& UnusedParameters)
{
	TStrongObjectPtr<UAIAssistantWebJavaScriptResultDelegate> ResultDelegate(
		NewObject<UAIAssistantWebJavaScriptResultDelegate>());
	int CallCount = 0;
	FString HandlerId = ResultDelegate->RegisterResultHandlerForCallback(
		[&CallCount](UAIAssistantWebJavaScriptResultDelegate::FResult) mutable ->
			UAIAssistantWebJavaScriptResultDelegate::EHandlerStatus
		{
			CallCount++;
			return UAIAssistantWebJavaScriptResultDelegate::EHandlerStatus::Continue;
		});
	const int ExpectedCalls = 3;
	for (int i = 0; i < ExpectedCalls; ++i)
	{
		FWebJavaScriptResultDelegateAccessor::CallHandleResult(
			*ResultDelegate, HandlerId, TEXT(""), false);
		(void)TestEqual(
			TEXT("ExpectedNumberOfCalls"), i + 1, CallCount);
	}
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantWebJavaScriptResultDelegateTestUnregisterResultHandler,
	"AI.Assistant.WebJavaScriptResultDelegate.UnregisterResultHandler",
	AIAssistantTest::Flags);

bool FAIAssistantWebJavaScriptResultDelegateTestUnregisterResultHandler::RunTest(
	const FString& UnusedParameters)
{
	TStrongObjectPtr<UAIAssistantWebJavaScriptResultDelegate> ResultDelegate(
		NewObject<UAIAssistantWebJavaScriptResultDelegate>());
	bool bWasCalled = false;
	FString HandlerId = ResultDelegate->RegisterResultHandler(
		[&bWasCalled](UAIAssistantWebJavaScriptResultDelegate::FResultHandlerContext&&)
			-> UAIAssistantWebJavaScriptResultDelegate::EHandlerStatus
		{
			bWasCalled = true;
			return UAIAssistantWebJavaScriptResultDelegate::EHandlerStatus::Unregister;
		});
	ResultDelegate->UnregisterResultHandler(HandlerId);
	(void)TestEqual(
		TEXT("NumberOfHandlers"), 0,
		FWebJavaScriptResultDelegateAccessor::GetResultHandlersById(
			*ResultDelegate).Num());
	FWebJavaScriptResultDelegateAccessor::CallHandleResult(
		*ResultDelegate, HandlerId, TEXT(""), false);
	(void)TestFalse(TEXT("HandlerNotCalled"), bWasCalled);
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantWebJavaScriptResultDelegateTestCompleteAllPendingPromises,
	"AI.Assistant.WebJavaScriptResultDelegate.CompleteAllPendingPromises",
	AIAssistantTest::Flags);

bool FAIAssistantWebJavaScriptResultDelegateTestCompleteAllPendingPromises::RunTest(
	const FString& UnusedParameters)
{
	FFakeWebJavaScriptDelegateBinder Binder;
	TStrongObjectPtr<UAIAssistantWebJavaScriptResultDelegate> ResultDelegate(
		NewObject<UAIAssistantWebJavaScriptResultDelegate>());
	ResultDelegate->Bind(Binder);
	TPair<FString, TFuture<UAIAssistantWebJavaScriptResultDelegate::FResult>> ResultHandle =
		ResultDelegate->RegisterResultHandlerForFuture();

	// Use Unbind() to force clean up of pending promises as we can't force destruction of the
	// UObject.
	ResultDelegate->Unbind();

	(void)TestTrue(TEXT("FutureIsReady"), ResultHandle.Value.IsReady());
	auto Result = ResultHandle.Value.Consume();
	(void)TestEqual(
		TEXT("FutureIsCanceled"),
		UAIAssistantWebJavaScriptResultDelegate::CanceledError,
		Result.Json);
	(void)TestTrue(TEXT("FutureHasError"), Result.bJsonIsError);
	return true;
}

DECLARE_MULTICAST_DELEGATE(FAIAssistantTestOnPreExit)

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FAIAssistantWebJavaScriptResultDelegateTestCompleteAllPendingPromisesOnPreExit,
	"AI.Assistant.WebJavaScriptResultDelegate.CompleteAllPendingPromisesOnPreExit",
	AIAssistantTest::Flags);

bool FAIAssistantWebJavaScriptResultDelegateTestCompleteAllPendingPromisesOnPreExit::RunTest(
	const FString& UnusedParameters)
{
	FAIAssistantTestOnPreExit OnPreExit;
	FFakeWebJavaScriptDelegateBinder Binder;
	TStrongObjectPtr<UAIAssistantWebJavaScriptResultDelegate> ResultDelegate(
		NewObject<UAIAssistantWebJavaScriptResultDelegate>());
	ResultDelegate->Bind(Binder, OnPreExit);
	TPair<FString, TFuture<UAIAssistantWebJavaScriptResultDelegate::FResult>> ResultHandle =
		ResultDelegate->RegisterResultHandlerForFuture();

	// Use OnPreExit() to force clean up of pending promises.
	OnPreExit.Broadcast();

	(void)TestTrue(TEXT("FutureIsReady"), ResultHandle.Value.IsReady());
	auto Result = ResultHandle.Value.Consume();
	(void)TestEqual(
		TEXT("FutureIsCanceled"),
		UAIAssistantWebJavaScriptResultDelegate::CanceledError,
		Result.Json);
	(void)TestTrue(TEXT("FutureHasError"), Result.bJsonIsError);
	return true;
}

#endif  // WITH_DEV_AUTOMATION_TESTS