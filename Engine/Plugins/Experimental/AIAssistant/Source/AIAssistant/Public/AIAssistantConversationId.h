// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Serialization/JsonSerializable.h"
#include "Serialization/JsonSerializerMacros.h"
#include "Templates/UnrealTemplate.h"

namespace UE::AIAssistant
{
	/** Identifies a conversation on the EDA backend. */
	struct FConversationId : public FJsonSerializable
	{
		/** Unique ID string, generated by the backend. */
		FString Id;
		/** Type discriminator. Always "ConversationId" for values produced by MakeConversationId(). */
		FString Type;

		bool operator==(const FConversationId& Other) const
		{
			return Id == Other.Id && Type == Other.Type;
		}

		BEGIN_JSON_SERIALIZER
			JSON_SERIALIZE("id", Id);
			JSON_SERIALIZE("type", Type);
		END_JSON_SERIALIZER
	};

	/**
	 * Creates a FConversationId from a string ID.
	 * @param Id  The conversation ID string produced by the backend.
	 * @return    A FConversationId with Type set to "ConversationId".
	 */
	inline FConversationId MakeConversationId(FString Id)
	{
		FConversationId ConvId;
		ConvId.Id = MoveTemp(Id);
		ConvId.Type = TEXT("ConversationId");
		return ConvId;
	}
}

FORCEINLINE uint32 GetTypeHash(const UE::AIAssistant::FConversationId& Id)
{
	return HashCombine(GetTypeHash(Id.Id), GetTypeHash(Id.Type));
}
